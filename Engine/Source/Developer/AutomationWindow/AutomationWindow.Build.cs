// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AutomationWindow : ModuleRules
	{
		public AutomationWindow(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"AutomationController",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
                    "DesktopPlatform",
                    "InputCore",
					"ApplicationCore",
                    "Slate",
                    "SlateCore",
                    "ToolWidgets",
                    "CoreUObject",
                    "Json",
                    "JsonUtilities",
					"SourceControl",
					"AutomationTest"
				}
			);

            // Added more direct dependencies to the editor for testing functionality
            if (Target.bBuildEditor)
            {
                PrivateDependencyModuleNames.AddRange(
                    new string[] {
						"EditorFramework",
                        "UnrealEd",
						"Kismet",
						"Engine", // Needed for UWorld/GWorld to find current level
				    }
                );
            }

            PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"SessionServices",
				}
			);
		}
	}
}
