// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationPresetManager.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "UObject/Class.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "JsonObjectConverter.h"
#include "SourceControlHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomationPresetManager)
DEFINE_LOG_CATEGORY_STATIC(LogAutomationPreset, Warning, All);

FAutomationTestPresetManager::FAutomationTestPresetManager()
{
	// Add the None Option
	Presets.Add(AutomationPresetPtr());
}

AutomationPresetPtr FAutomationTestPresetManager::AddNewPreset( const FText& PresetName, const TArray<FString>& SelectedTests )
{
	if ( PresetName.IsEmpty() )
	{
		return nullptr;
	}

	const FName NewNameSlug = MakeObjectNameFromDisplayLabel(PresetName.ToString(), NAME_None);

	if ( !Presets.FindByPredicate([&NewNameSlug] (const AutomationPresetPtr& Preset) { return Preset.IsValid() && Preset->GetID() == NewNameSlug; }) )
	{
		AutomationPresetRef NewPreset = MakeShareable(new FAutomationTestPreset(NewNameSlug));
		NewPreset->SetName(PresetName);
		NewPreset->SetEnabledTests(SelectedTests);

		Presets.Add(NewPreset);

		SavePreset(NewPreset);

		return NewPreset;
	}

	return nullptr;
}

TArray<AutomationPresetPtr>& FAutomationTestPresetManager::GetAllPresets()
{
	return Presets;
}

AutomationPresetPtr FAutomationTestPresetManager::LoadPreset( FArchive& Archive )
{
	TSharedPtr<FJsonObject> JsonPreset = MakeShared<FJsonObject>();

	FString JsonContent;
	if ( !FFileHelper::LoadFileToString(JsonContent, Archive) )
	{
		return nullptr;
	}

	TSharedRef< TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(JsonContent);
	if ( !FJsonSerializer::Deserialize(JsonReader, JsonPreset) )
	{
		return nullptr;
	}

	FAutomationTestPreset* NewPreset = new FAutomationTestPreset();

	if ( FJsonObjectConverter::JsonObjectToUStruct(JsonPreset.ToSharedRef(), NewPreset, 0, 0) )
	{
		return MakeShareable(NewPreset);
	}

	delete NewPreset;

	return nullptr;
}

void FAutomationTestPresetManager::RemovePreset( const AutomationPresetRef Preset )
{
	if (Presets.Remove(Preset) > 0)
	{
		// Find the name of the preset on disk
		FString PresetFileName = GetPresetFolder() / Preset->GetID().ToString() + TEXT(".json");

		// delete the preset on disk
		IFileManager::Get().Delete(*PresetFileName);
	}
}

bool FAutomationTestPresetManager::SavePreset(const AutomationPresetRef Preset)
{
	FString JsonString;
	if (!FJsonObjectConverter::UStructToJsonObjectString(Preset.Get(), JsonString))
	{
		UE_LOGF(LogAutomationPreset, Error, "Could not convert preset file '%ls' to JSON string", *Preset->GetName().ToString());
		return false;
	}

	const FString PresetFileName = GetPresetFolder() / Preset->GetID().ToString() + TEXT(".json");
	bool bShouldMarkForAdd = false;

	if (IFileManager::Get().FileExists(*PresetFileName))
	{
		if (USourceControlHelpers::IsEnabled())
		{
			// Checkout (or add in case file exists locally but not in source control).
			USourceControlHelpers::CheckOutOrAddFile(PresetFileName);
		}
	}
	else
	{
		bShouldMarkForAdd = true;
	}

	if (!FFileHelper::SaveStringToFile(JsonString, *PresetFileName, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
	{
		UE_LOGF(LogAutomationPreset, Error, "Could not save preset file '%ls'", *Preset->GetID().ToString());
		return false;
	}

	if (bShouldMarkForAdd)
	{
		if (USourceControlHelpers::IsEnabled())
		{
			// Mark for add (or checkout in case file already exists in source control).
			USourceControlHelpers::CheckOutOrAddFile(PresetFileName);
		}
	}
	return true;
}

void FAutomationTestPresetManager::LoadPresets()
{
	TArray<FString> PresetFileNames;

	IFileManager::Get().FindFiles(PresetFileNames, *(GetPresetFolder() / TEXT("*.json")), true, false);

	for (TArray<FString>::TConstIterator It(PresetFileNames); It; ++It)
	{
		FString PresetFilePath = GetPresetFolder() / *It;
		FArchive* PresetFileReader = IFileManager::Get().CreateFileReader(*PresetFilePath);

		if (PresetFileReader != nullptr)
		{
			AutomationPresetPtr LoadedPreset = LoadPreset(*PresetFileReader);

			if (LoadedPreset.IsValid())
			{
				Presets.Add(LoadedPreset);
			}
			else
			{
				UE_LOGF(LogAutomationPreset, Warning, "Could not read preset file '%ls'. Make sure the file is encoded in UTF-8 without BOM.", *PresetFilePath);
			}

			delete PresetFileReader;
		}
	}
}

