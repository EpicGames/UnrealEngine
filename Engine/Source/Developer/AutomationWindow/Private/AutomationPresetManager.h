// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Paths.h"
#include "AutomationPresetManager.generated.h"

struct FAutomationTestPreset;

/** Type definition for shared pointers to instances of FAutomationTestPreset. */
typedef TSharedPtr<struct FAutomationTestPreset> AutomationPresetPtr;

/** Type definition for shared references to instances of FAutomationTestPreset. */
typedef TSharedRef<struct FAutomationTestPreset> AutomationPresetRef;

/**
 * Class that holds preset data for the automation window
 */
USTRUCT()
struct FAutomationTestPreset
{
	GENERATED_BODY()

public:

	/** Default constructor. */
	FAutomationTestPreset()
	{
	}

	/** New Preset constructor. */
	FAutomationTestPreset(FName NewPresetId)
		: Id(NewPresetId)
	{
	}

	/** Destructor. */
	virtual ~FAutomationTestPreset( ) = default;

	/**
	 * Gets the unique preset id for this preset.
	 */
	FName GetID() const
	{
		return Id;
	}

	/**
	 * Gets the name for this preset.
	 *
	 * @return The preset's name.
	 * @see GetID, SetPresetName
	 */
	const FText& GetName() const
	{
		return Name;
	}

	/**
	 * Sets the name for this preset.
	 *
	 * @param InPresetName The name to set.
	 * @see GetPresetName
	 */
	void SetName(const FText& InPresetName)
	{
		Name = InPresetName;
	}

	/**
	 * Returns the list of enabled tests.
	 *
	 * @return Collection of tests.
	 * @see SetEnabledTests
	 */
	const TArray<FString>& GetEnabledTests() const
	{
		return EnabledTests;
	}

	/**
	 * Sets the list of enabled tests.
	 *
	 * @param NewEnableDtests The collection of tests to enable.
	 * @see GetEnabledTests
	 */
	void SetEnabledTests(TArray<FString> NewEnabledTests)
	{
		EnabledTests = NewEnabledTests;
	}

private:

	/** The unique name for this preset. */
	UPROPERTY()
	FName Id;

	/** The name of this preset. */
	UPROPERTY()
	FText Name;

	/** The list of enabled test names. */
	UPROPERTY()
	TArray<FString> EnabledTests;
};


class FAutomationTestPresetManager
{
public:

	/** Default constructor. */
	FAutomationTestPresetManager();

	/** Virtual destructor. */
	virtual ~FAutomationTestPresetManager() = default;

	/**
	 * Creates a new preset with the given name and enabled tests.
	 * @param PresetName The name of the new preset.
	 * @param SelectedTests The list of enabled tests.
	 */
	virtual AutomationPresetPtr AddNewPreset( const FText& PresetName, const TArray<FString>& SelectedTests );

	/**
	 * Returns a reference to the list that holds the presets.
	 */
	virtual TArray<AutomationPresetPtr>& GetAllPresets( );

	/**
	 * Removes the selected preset from the preset list and deletes the file on disk.
	 *
	 * @param Preset The preset to remove.
	 * @see LoadPresets, SavePreset
	 */
	void RemovePreset( const AutomationPresetRef Preset );

	/**
	 * Saves the passed in preset to disk.
	 *
	 * @param Preset The preset to save.
	 * @see RemovePreset, LoadPresets
	 */
	bool SavePreset( const AutomationPresetRef Preset);

	/**
	 * Load all Presets from disk.
	 *
	 * @see RemovePreset, SavePreset
	 */
	void LoadPresets( );

protected:

	/**
	 * Creates a new preset and loads it with data from the archive.
	 *
	 * @param Archive The stream to read the preset data from.
	 */
	AutomationPresetPtr LoadPreset( FArchive& Archive );

	/**
	 * Gets the folder in which preset files are stored.
	 *
	 * @return The folder path.
	 */
	static FString GetPresetFolder()
	{
		return FPaths::ProjectConfigDir() / TEXT("Automation/Presets");
	}

private:

	/** Holds the collection of automation presets. */
	TArray<AutomationPresetPtr> Presets;
};
