// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "ISessionManager.h"
#include "SAutomationWindow.h"

#include "IAutomationWindowModule.h"
#include "AutomationWindowStyle.h"


/**
 * Implements the AutomationWindow module.
 */
class FAutomationWindowModule
	: public IAutomationWindowModule
{
public:

	// IAutomationWindowModule interface

	virtual TSharedRef<class SWidget> CreateAutomationWindow( const IAutomationControllerManagerRef& AutomationController, const TSharedRef<ISessionManager>& SessionManager ) override
	{
		return SNew(SAutomationWindow, AutomationController, SessionManager);
	}

	virtual TWeakPtr<class SDockTab> GetAutomationWindowTab( ) override
	{
		return AutomationWindowTabPtr;
	}

	virtual FOnAutomationWindowModuleShutdown& OnShutdown( ) override
	{
		return ShutdownDelegate;
	}

	virtual void SetAutomationWindowTab(TWeakPtr<class SDockTab> AutomationWindowTab) override { AutomationWindowTabPtr = AutomationWindowTab; }

public:

	// IModuleInterface interface

	virtual void StartupModule( ) override
	{
		FAutomationWindowStyle::Get();
	}

	virtual void ShutdownModule( ) override
	{
		FAutomationWindowStyle::Shutdown();
		ShutdownDelegate.ExecuteIfBound();	
	}

private:

	// Holds the DockTab for the AutomationWindow
	TWeakPtr<class SDockTab> AutomationWindowTabPtr;

	// Holds FAutomationWindowModuleShutdownCallback
	FOnAutomationWindowModuleShutdown ShutdownDelegate;
};


IMPLEMENT_MODULE(FAutomationWindowModule, AutomationWindow);
