// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationWindowStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/CoreStyle.h"

FName FAutomationWindowStyle::StyleName("AutomationWindowStyle");
TUniquePtr<FAutomationWindowStyle> FAutomationWindowStyle::Inst(nullptr);

const FName& FAutomationWindowStyle::GetStyleSetName() const
{
	return StyleName;
}

const FAutomationWindowStyle& FAutomationWindowStyle::Get()
{
	if (!Inst.IsValid())
	{
		Inst = TUniquePtr<FAutomationWindowStyle>(new FAutomationWindowStyle);
	}
	return *(Inst.Get());
}

void FAutomationWindowStyle::Shutdown()
{
	Inst.Reset();
}

FAutomationWindowStyle::FAutomationWindowStyle()
	: FSlateStyleSet(StyleName)
{
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);

	SetParentStyleName(FAppStyle::GetAppStyleSetName());

	SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	{
		FTextBlockStyle NormalText = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");

		Set("Automation.Header", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Mono", 12))
			.SetColorAndOpacity(FStyleColors::ForegroundHeader));

		Set("Automation.Normal", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Mono", 9))
			.SetColorAndOpacity(FStyleColors::Foreground));

		Set("Automation.Warning", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Mono", 9))
			.SetColorAndOpacity(FStyleColors::Warning));

		Set("Automation.Error", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Mono", 9))
			.SetColorAndOpacity(FStyleColors::Error));

		Set("Automation.ReportHeader", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Mono", 10))
			.SetColorAndOpacity(FStyleColors::ForegroundHeader));

		//state of individual tests
		Set("Automation.Success", new CORE_IMAGE_BRUSH("Automation/Success", Icon16x16));
		Set("Automation.Warning", new CORE_IMAGE_BRUSH("Automation/Warning", Icon16x16));
		Set("Automation.Fail", new CORE_IMAGE_BRUSH("Automation/Fail", Icon16x16));
		Set("Automation.InProcess", new CORE_IMAGE_BRUSH("Automation/InProcess", Icon16x16));
		Set("Automation.NotRun", new CORE_IMAGE_BRUSH("Automation/NotRun", Icon16x16, FLinearColor(0.0f, 0.0f, 0.0f, 0.4f)));
		Set("Automation.Skipped", new CORE_IMAGE_BRUSH("Automation/NoSessionWarning", Icon16x16));
		Set("Automation.ParticipantsWarning", new CORE_IMAGE_BRUSH_SVG("Starship/Common/PlayerController", Icon16x16));
		Set("Automation.Participant", new CORE_IMAGE_BRUSH_SVG("Starship/Common/PlayerController", Icon16x16));

		//status as a regression test or not
		Set("Automation.SmokeTest", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Test", Icon16x16));
		Set("Automation.SmokeTestParent", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Test", Icon16x16));

		//run icons
		Set("AutomationWindow.RunTests", new CORE_IMAGE_BRUSH_SVG("Starship/Common/play", Icon20x20));
		Set("AutomationWindow.RunLevelTest", new IMAGE_BRUSH_SVG("Starship/Common/RunLevelTest", Icon20x20));
		Set("AutomationWindow.RefreshTests", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Update", Icon20x20));
		Set("AutomationWindow.FindWorkers", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Search_20", Icon20x20));
		Set("AutomationWindow.StopTests", new CORE_IMAGE_BRUSH_SVG("Starship/Common/stop", Icon20x20));
		Set("AutomationWindow.RunTests.Small", new CORE_IMAGE_BRUSH_SVG("Starship/Common/play", Icon20x20));
		Set("AutomationWindow.RunLevelTest.Small", new IMAGE_BRUSH_SVG("Starship/Common/RunLevelTest", Icon20x20));
		Set("AutomationWindow.RefreshTests.Small", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Update", Icon20x20));
		Set("AutomationWindow.FindWorkers.Small", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Search_20", Icon20x20));
		Set("AutomationWindow.StopTests.Small", new CORE_IMAGE_BRUSH_SVG("Starship/Common/stop", Icon20x20));

		//filter icons
		Set("AutomationWindow.ErrorFilter", new CORE_IMAGE_BRUSH_SVG("Starship/Common/x-circle", Icon16x16));
		Set("AutomationWindow.WarningFilter", new CORE_IMAGE_BRUSH_SVG("Starship/Common/alert-triangle", Icon16x16));
		Set("AutomationWindow.SmokeTestFilter", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Test", Icon16x16));
		Set("AutomationWindow.DeveloperDirectoryContent", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Developer", Icon16x16));
		Set("AutomationWindow.ExcludedTestsFilter", new CORE_IMAGE_BRUSH_SVG("Starship/Common/reject", Icon16x16));
		Set("AutomationWindow.EditTags", new CORE_IMAGE_BRUSH_SVG("Starship/Common/TagSimple_16", Icon16x16));
		Set("AutomationWindow.ApplyTagFilter", new CORE_IMAGE_BRUSH_SVG("Starship/Common/filter", Icon16x16));
		Set("AutomationWindow.TagFilterIsActive", new CORE_IMAGE_BRUSH_SVG("Starship/Common/badge-modified", Icon16x16));
		Set("AutomationWindow.RemoveTagFilter", new CORE_IMAGE_BRUSH_SVG("Starship/Common/close", Icon16x16));
		Set("AutomationWindow.ErrorFilter.Small", new CORE_IMAGE_BRUSH_SVG("Starship/Common/x-circle", Icon16x16));
		Set("AutomationWindow.WarningFilter.Small", new CORE_IMAGE_BRUSH_SVG("Starship/Common/alert-triangle", Icon16x16));
		Set("AutomationWindow.SmokeTestFilter.Small", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Test", Icon16x16));
		Set("AutomationWindow.DeveloperDirectoryContent.Small", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Developer", Icon16x16));

		//device group settings
		Set("AutomationWindow.GroupSettings", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Group_20", Icon20x20));
		Set("AutomationWindow.GroupSettings.Small", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Group_20", Icon20x20));

		//test backgrounds
		Set("AutomationWindow.GameGroupBorder", new CORE_BOX_BRUSH("Automation/GameGroupBorder", FMargin(4.0f / 16.0f)));
		Set("AutomationWindow.EditorGroupBorder", new CORE_BOX_BRUSH("Automation/EditorGroupBorder", FMargin(4.0f / 16.0f)));

		FButtonStyle Button = FButtonStyle()
			.SetNormal(BOX_BRUSH("Common/Button", FVector2D(32, 32), 8.0f / 32.0f))
			.SetHovered(BOX_BRUSH("Common/Button_Hovered", FVector2D(32, 32), 8.0f / 32.0f))
			.SetPressed(BOX_BRUSH("Common/Button_Pressed", FVector2D(32, 32), 8.0f / 32.0f))
			.SetNormalPadding(FMargin(2, 2, 2, 2))
			.SetPressedPadding(FMargin(2, 3, 2, 1));

		Set("AutomationWindow.ToggleButton", FCheckBoxStyle(FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
			.SetUncheckedHoveredImage(FSlateColorBrush(FStyleColors::SelectHover))
			.SetPadding(FMargin(16, 4))
		);
	}
	
	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAutomationWindowStyle::~FAutomationWindowStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
