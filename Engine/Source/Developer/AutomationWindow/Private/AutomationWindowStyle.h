// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateColor.h"
#include "UObject/NameTypes.h"
#include "Templates/UniquePtr.h"

class FAutomationWindowStyle
	: public FSlateStyleSet
{
public:
	virtual const FName& GetStyleSetName() const override;

	static const FAutomationWindowStyle& Get();
	static void Shutdown();
	
	~FAutomationWindowStyle();

private:
	FAutomationWindowStyle();

	static FName StyleName;
	static TUniquePtr<FAutomationWindowStyle> Inst;
};
