// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAutomationExportMenu.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "IAutomationControllerModule.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Notifications/SNotificationList.h"


#define LOCTEXT_NAMESPACE "AutomationExportMenu"


SAutomationExportMenu::SAutomationExportMenu()
{
	// Set up some variables
	FileExportTypeMask = 0;
	EFileExportType::SetFlag( FileExportTypeMask, EFileExportType::FET_All );
}


void SAutomationExportMenu::Construct( const FArguments& InArgs, const TSharedRef< SNotificationList >& InNotificationList )
{
	// Used for setting the "Exported" notification on the parent window
	NotificationListPtr = InNotificationList;

	// Build the UI
	ChildSlot
	[
		SAssignNew( ExportMenuComboButton, SComboButton )
		.IsEnabled( this, &SAutomationExportMenu::AreReportsGenerated )
		.ToolTipText( this, &SAutomationExportMenu::GetExportComboButtonTooltip )
		.OnComboBoxOpened( this, &SAutomationExportMenu::HandleMenuOpen )
		.ButtonContent()
		[
			SNew( STextBlock )
			.Text( LOCTEXT("ExportButtonText", "Export") )
		]
		.ContentPadding( FMargin( 6.f, 2.f ) )
		.MenuContent()
		[
			// Holder box for menu items
			SAssignNew( MenuHolderBox, SVerticalBox )
		]
	];
}


bool SAutomationExportMenu::AreReportsGenerated() const
{
	// Check with the manager to see if reports are ready
	return FModuleManager::GetModuleChecked< IAutomationControllerModule >( "AutomationController" ).GetAutomationController()->CheckTestResultsAvailable();
}


void SAutomationExportMenu::BuildMenuItems( const FText& InName, EFileExportType::Type InType )
{
	MenuHolderBox->AddSlot()
	.AutoHeight()
	[
		SNew( SCheckBox )
		.IsChecked( this, &SAutomationExportMenu::OnGetDisplayCheckState, InType )
		.IsEnabled( IsCheckBoxEnabled( InType ) )
		.OnCheckStateChanged( this, &SAutomationExportMenu::OnDisplayCheckStateChanged, InType )
		[
			SNew( SHorizontalBox )
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew( STextBlock ) .Text( InName )
			]
		]
	];
}

void SAutomationExportMenu::CreateMenu()
{
	// Clear old menu
	MenuHolderBox->ClearChildren();

	// Create new menu items
	BuildMenuItems( LOCTEXT("ExportAllCheckbox", "Export All"), EFileExportType::FET_All );
	BuildMenuItems( LOCTEXT("ExportStatusCheckBox", "Export Status"), EFileExportType::FET_Status );
	BuildMenuItems( LOCTEXT("ExportErrorsCheckBox", "Export Errors"), EFileExportType::FET_Errors );
	BuildMenuItems( LOCTEXT("ExportWarningsCheckBox", "Export Warning"), EFileExportType::FET_Warnings );
	BuildMenuItems( LOCTEXT("ExportLogsCheckBox", "Export Logs"), EFileExportType::FET_Logs );

	// Add the export button
	MenuHolderBox->AddSlot()
	.AutoHeight()
	[
		SAssignNew( ExportButton, SButton )
		.OnClicked( this, &SAutomationExportMenu::HandleExportDataClicked )
		.IsEnabled( this, &SAutomationExportMenu::IsExportReady )
		.ToolTipText( this, &SAutomationExportMenu::GetExportButtonTooltip )
		.Text( LOCTEXT("ExportDataButton", "Export Data") )
	];
}


FText SAutomationExportMenu::GetExportButtonTooltip() const
{
	// Export button tooltip
	if ( ExportButton->IsEnabled() )
	{
		return LOCTEXT("ExportButtonEnabledText", "Export Data");
	}
	return LOCTEXT("ExportButtonFailedText", "No reports pass the export filter");
}


FText SAutomationExportMenu::GetExportComboButtonTooltip() const
{
	// Export combo too tip
	if ( ExportMenuComboButton->IsEnabled() )
	{
		return LOCTEXT("ExportComboButtonEnabledText", "Export Data");
	}
	return LOCTEXT("ExportComboButtonFailedText", "Please generate the reports");
}


void SAutomationExportMenu::GetResults( )
{
	// Get the automation controller to check what type of reports have been generated
	IAutomationControllerManagerPtr AutomationController = FModuleManager::GetModuleChecked< IAutomationControllerModule >( "AutomationController" ).GetAutomationController();

	// Reset the results mask
	ResultMask = 0;

	// Add result flags

	// Always allow status and all
	EFileExportType::SetFlag( ResultMask, EFileExportType::FET_Status );

	EFileExportType::SetFlag( ResultMask, EFileExportType::FET_All );

	if ( AutomationController->ReportsHaveErrors() )
	{
		EFileExportType::SetFlag( ResultMask, EFileExportType::FET_Errors );
	}

	if ( AutomationController->ReportsHaveWarnings() )
	{
		EFileExportType::SetFlag( ResultMask, EFileExportType::FET_Warnings );
	}

	if ( AutomationController->ReportsHaveLogs())
	{
		EFileExportType::SetFlag( ResultMask, EFileExportType::FET_Warnings );
	}
}


FReply SAutomationExportMenu::HandleExportDataClicked()
{
	// Time to display the completion message in the UI
	static const float MessageTime = 3.0f;

	// Generate the report
	bool ReportExported = FModuleManager::GetModuleChecked< IAutomationControllerModule >( "AutomationController" ).GetAutomationController()->ExportReport( FileExportTypeMask );

	// Notify UI depending on whether the report was generated
	if (ReportExported)
	{
		// Create the file name. This is currently duplicated in the report manager
		FFormatNamedArguments Args;
		Args.Add( TEXT("FileName"), FText::FromString( FString::Printf( TEXT("Automation%s.csv"), *FDateTime::Now().ToString() ) ) );
		Args.Add( TEXT("FileLocation"), FText::FromString( FPaths::ConvertRelativePathToFull( FPaths::AutomationDir() ) ) );

		// Feedback that the file is written
		FNotificationInfo NotificationInfo( FText::Format( LOCTEXT("SaveLogDialogExportSuccess", "Success!\n{FileName} exported to: {FileLocation}"), Args ) );
		NotificationInfo.ExpireDuration = MessageTime;
		NotificationInfo.bUseLargeFont = false;
		NotificationListPtr->AddNotification( NotificationInfo );
	}
	else
	{
		FNotificationInfo NotificationInfo( LOCTEXT("SaveLogDialogNothingValidError", "No reports pass the export filter") );
		NotificationInfo.ExpireDuration = MessageTime;

		// No file generated as there are no reports that pass the filters
		NotificationListPtr->AddNotification( NotificationInfo );
	}

	// close the export window
	ExportMenuComboButton->SetIsOpen( false );

	return FReply::Handled();
}


void SAutomationExportMenu::HandleMenuOpen()
{
	// Get the results from the automation controller
	GetResults();

	// Set all flags is Set All is selected
	if ( EFileExportType::IsSet( FileExportTypeMask, EFileExportType::FET_All ) )
	{
		EnableAvailableReports();
	}

	// Create the menu items
	CreateMenu();
}


bool SAutomationExportMenu::IsCheckBoxEnabled( EFileExportType::Type CheckType ) const
{
	// Enable the button if there is a valid report
	return EFileExportType::IsSet( ResultMask, CheckType);
}


bool SAutomationExportMenu::IsExportReady() const
{
	// If we have a valid export mask, we can assume we are able to export a report
	return  FileExportTypeMask > 0;
}


void SAutomationExportMenu::OnDisplayCheckStateChanged( ECheckBoxState InNewState, EFileExportType::Type CheckType )
{
	// set or unset the bit in the mask for the type changed
	if ( InNewState == ECheckBoxState::Checked )
	{
		EFileExportType::SetFlag( FileExportTypeMask, CheckType );
	}
	else
	{
		EFileExportType::RemoveFlag( FileExportTypeMask, CheckType );
	}

	// If select all is selected, and we select something else, unset the "all selected" checkbox
	if ( CheckType != EFileExportType::FET_All && EFileExportType::IsSet( FileExportTypeMask, EFileExportType::FET_All ) )
	{
		EFileExportType::RemoveFlag( FileExportTypeMask, EFileExportType::FET_All );
		// Recreate the menu
		CreateMenu();
	}

	// If we check select all, set all the valid options to be active
	if ( CheckType == EFileExportType::FET_All && InNewState == ECheckBoxState::Checked )
	{
		EnableAvailableReports();
		// Recreate the menu
		CreateMenu();
	}
}


ECheckBoxState SAutomationExportMenu::OnGetDisplayCheckState( EFileExportType::Type CheckType ) const
{
	// If the item is selected, set the box to be checked
	if(EFileExportType::IsSet( FileExportTypeMask, CheckType ) )
	{
		return ECheckBoxState::Checked;
	}
	return ECheckBoxState::Unchecked;
}


void SAutomationExportMenu::EnableAvailableReports()
{
	// Set all valid reports to be exported
	EFileExportType::SetFlag( FileExportTypeMask, EFileExportType::FET_Status );

	if ( EFileExportType::IsSet( ResultMask, EFileExportType::FET_Errors ) )
	{
		EFileExportType::SetFlag( FileExportTypeMask, EFileExportType::FET_Errors );
	}
	if ( EFileExportType::IsSet( ResultMask, EFileExportType::FET_Warnings ) )
	{
		EFileExportType::SetFlag( FileExportTypeMask, EFileExportType::FET_Warnings );
	}
	if ( EFileExportType::IsSet( ResultMask, EFileExportType::FET_Logs ) )
	{
		EFileExportType::SetFlag( FileExportTypeMask, EFileExportType::FET_Logs );
	}
}


FReply SAutomationExportMenu::SpawnNotification()
{
	// Inform the UI that we have generated the report
	NotificationListPtr->AddNotification( FNotificationInfo( LOCTEXT("ReportGeneratedSuccessfullyNotification", "Report Generated Successfully!" )) );
	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE
