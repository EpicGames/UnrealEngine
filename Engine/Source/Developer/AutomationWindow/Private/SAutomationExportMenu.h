// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "IAutomationControllerManager.h"

class SButton;
class SComboButton;
class SVerticalBox;
enum class ECheckBoxState : uint8;

class SAutomationExportMenu : public SCompoundWidget
{
public:

	/** Default constructor. */
	SAutomationExportMenu();

	SLATE_BEGIN_ARGS( SAutomationExportMenu ){}
	SLATE_END_ARGS()

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs	Declaration used by the SNew() macro to construct this widget
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<SNotificationList>& InNotificationList );

private:

	/**
	 * Have the reports been generated
	 *
	 * @return true if the reports have been generated
	 */
	bool AreReportsGenerated() const;

	/** 
	 * Build the menu items.
	 *
	 * @param InName The menu item name.
	 * @param InType The menu item type.
	 */
	void BuildMenuItems( const FText& InName, EFileExportType::Type InType );

	/** Create the menu widgets. */
	void CreateMenu( );

	/**
	 * Set all the available export options to checked.
	 */
	void EnableAvailableReports();

	/**
	 * Get the export button tooltip.
	 *
	 * @return true export report is ready.
	 */
	FText GetExportButtonTooltip() const;

	/**
	 * Get the export combo button tooltip.
	 *
	 * @return true export report is ready.
	 */
	FText GetExportComboButtonTooltip() const;

	/** Get the test result summary from the manager. */
	void GetResults( );

	/** 
	* Export the report data when clicked.
	*
	* @return handled if the report has been handled.
	*/
	FReply HandleExportDataClicked();

	/**
	* Called when the menu button is opened. Gets the results, and generates the widgets.
	*/
	void HandleMenuOpen();

	/** 
	 * Should the export checkboxes be enabled.
	 *
	 * @param CheckType The menu item type.
	 * @return true if the checkbox is enabled .
	 */
	bool IsCheckBoxEnabled( EFileExportType::Type CheckType ) const;

	/**
	* Is an export report ready.
	*
	* @return true export report is ready.
	*/
	bool IsExportReady() const;

	/** 
	 * Check box has changed, update the export data mask .
	 *
	 * @param The new state, checked or unchecked.
	 * @param The type of checkbox set.
	 */
	void OnDisplayCheckStateChanged( ECheckBoxState InNewState, EFileExportType::Type CheckType );

	/** 
	 * Gets the display state to send to a display filter check box
	 *
	 * @param The type of checkbox.
	 * @return The desired checkbox state.
	 */
	ECheckBoxState OnGetDisplayCheckState( EFileExportType::Type CheckType ) const;

	/**
	 * Add a notification when the file is exported.
	 *
	 * @return true export report is ready.
	 */
	FReply SpawnNotification();

private:

	/** The export button */
	TSharedPtr< SButton > ExportButton;

	/** The combo button */
	TSharedPtr< SComboButton > ExportMenuComboButton;

	/** Holds the file export mask for deciding which reports to output e.g. only errors */
	uint32 FileExportTypeMask;

	// Box to hold the menu items
	TSharedPtr< SVerticalBox > MenuHolderBox;

	/** The list of active system messages */
	TSharedPtr< SNotificationList > NotificationListPtr;

	/** Holds the result mask generated by the report manager */
	uint32 ResultMask;
};
