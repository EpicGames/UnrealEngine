// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAutomationGraphicalResultBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Styling/AppStyle.h"
#include "AutomationWindowStyle.h"

#define LOCTEXT_NAMESPACE "SAutomationGraphicalResultBox"


/* SAutomationGraphicalResultBox interface
 *****************************************************************************/

void SAutomationGraphicalResultBox::Construct( const FArguments& InArgs, const IAutomationControllerManagerRef& InAutomationController )
{
	AutomationController = InAutomationController;
	DisplayType = EAutomationGrapicalDisplayType::DisplayName;

	RootBox = SNew(SVerticalBox); 

	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			RootBox.ToSharedRef()
		]
	];

	ClearResults();

	AutomationController->OnTestsComplete().AddRaw(this, &SAutomationGraphicalResultBox::OnFillResults);
}

SAutomationGraphicalResultBox::~SAutomationGraphicalResultBox()
{
	AutomationController->OnTestsComplete().RemoveAll(this);
}

void SAutomationGraphicalResultBox::ClearResults()
{
	TotalTestDuration = 0.f;
	ClusterResults.Empty();
	RootBox->ClearChildren();
}


void SAutomationGraphicalResultBox::GetEnabledReports(TSharedPtr<IAutomationReport> InReport, TArray< TSharedPtr< IAutomationReport > >& outReports)
{
	TArray<TSharedPtr<IAutomationReport> >& ChildReports = InReport->GetChildReports();

	if (ChildReports.Num() > 0)
	{
		for (int32 Index = 0; Index < ChildReports.Num(); Index++)
		{
			GetEnabledReports(ChildReports[Index], outReports);
		}
	}
	else if ( InReport->IsEnabled() )
	{
		outReports.Add(InReport);
	}
}


void SAutomationGraphicalResultBox::PopulateData()
{
	//Get the report list
	TArray< TSharedPtr< IAutomationReport > > AllReports;
	AllReports = AutomationController->GetEnabledReports();

	//Find only the enabled tests
	TArray< TSharedPtr< IAutomationReport > > EnabledReports;
	for( int32 i=0; i<AllReports.Num(); ++i )
	{
		GetEnabledReports(AllReports[i],EnabledReports);
	}

	uint32 LastPassIndex = AutomationController->GetNumPasses() - 1;

	//Pull the data out of the reports
	for(int i = 0; i < AutomationController->GetNumDeviceClusters(); ++i)
	{
		ClusterResults.Add(FClusterResults(AutomationController->GetClusterGroupName(i)));
		FClusterResults* ClusterIt = &ClusterResults[i];
		ClusterIt->TotalNumTests = EnabledReports.Num();
		for(int j = 0; j < AutomationController->GetNumDevicesInCluster(i); ++j)
		{
			ClusterIt->Devices.Add(FDeviceResults(AutomationController->GetGameInstanceId(i, j), AutomationController->GetGameInstanceName(i, j)));
			FDeviceResults* DeviceIt = &ClusterIt->Devices[j];
			for( int k = 0; k < EnabledReports.Num(); ++k )
			{
				FAutomationTestResults TestResults = EnabledReports[k]->GetResults( i, LastPassIndex );
				if (TestResults.GameInstance == DeviceIt->GameInstanceId)
				{
					uint32 TestIndex = DeviceIt->Tests.Add(FTestResults(EnabledReports[k]->GetDisplayNameWithDecoration()));
					FTestResults* TestIt = &DeviceIt->Tests[TestIndex];
					TestIt->Duration = TestResults.Duration;
					TestIt->TestState = TestResults.State;
					TestIt->bHasWarnings = TestResults.GetWarningTotal() > 0;

					DeviceIt->TotalTime += TestIt->Duration;
					ClusterIt->TotalTime += TestIt->Duration;

					if( TestResults.State == EAutomationState::Success )
					{
						DeviceIt->TotalTestSuccesses++;
						ClusterIt->TotalTestSuccesses++;
					}
					else if (TestResults.State == EAutomationState::Skipped)
					{
						DeviceIt->TotalTestSkips++;
						ClusterIt->TotalTestSkips++;
					}
				}
			}

			//See if this is the new longest running device for this cluster
			if( DeviceIt->TotalTime > ClusterIt->ParallelTime )
			{
				ClusterIt->ParallelTime = DeviceIt->TotalTime;
			}
		}

		//The total test duration is the time of the longest cluster.
		if( ClusterIt->ParallelTime > TotalTestDuration )
		{
			TotalTestDuration = ClusterIt->ParallelTime;
		}
	}
}


void SAutomationGraphicalResultBox::OnFillResults()
{
	ClearResults();
	PopulateData();
	CreateWidgets();
}


void SAutomationGraphicalResultBox::CreateWidgets()
{
	//Note: I am using 10 columns here to hack around a sizing bug.  The Grid calculates the width for items that span multiple 
	//		columns by just dividing the size by the number of columns.  This causes problems if the columns are different sizes.
	//		By having a bunch of extra columns here it will calculate a smaller per column size for our header.
	TSharedRef<SGridPanel> GridContainer = SNew(SGridPanel).FillColumn(1,1);

	uint32 RowCounter = 0;

	for( int32 ClusterIndex = 0; ClusterIndex < ClusterResults.Num(); ++ClusterIndex )
	{
		FClusterResults* ClusterIt = &ClusterResults[ClusterIndex];

		FFormatNamedArguments ClusterArgs;
		ClusterArgs.Add(TEXT("Name"), FText::FromString(ClusterIt->ClusterName));
		ClusterArgs.Add(TEXT("NumTests"), ClusterIt->TotalNumTests);
		ClusterArgs.Add(TEXT("NumSkips"), ClusterIt->TotalTestSkips);
		ClusterArgs.Add(TEXT("NumFails"), ClusterIt->TotalNumTests - ClusterIt->TotalTestSuccesses - ClusterIt->TotalTestSkips);
		ClusterArgs.Add(TEXT("TotalTime"), ClusterIt->TotalTime);
		ClusterArgs.Add(TEXT("ParallelTime"), ClusterIt->ParallelTime);

		//Add Cluster Header
		GridContainer->AddSlot(0,RowCounter)
			.ColumnSpan(10)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding( FMargin(1,3) )
			[
				SNew(STextBlock)
				.TextStyle(FAutomationWindowStyle::Get(), "Automation.ReportHeader" )
				.Text(FText::Format(LOCTEXT("AutomationGraphicalClusterHeader", "{Name}  -  {NumTests} Tests / {NumFails} Fails / {NumSkips} Skips / {TotalTime} Seconds (Total) / {ParallelTime} Seconds (Parallel)"), ClusterArgs))
			];

		RowCounter++;

		for( int32 DeviceIndex = 0; DeviceIndex < ClusterIt->Devices.Num(); ++DeviceIndex )
		{
			FDeviceResults* DeviceIt = &ClusterIt->Devices[DeviceIndex];
			const int32 NumTests = DeviceIt->Tests.Num();

			//Add Device Header
			FFormatNamedArguments DeviceArgs;
			DeviceArgs.Add(TEXT("NumTests"), NumTests);
			DeviceArgs.Add(TEXT("NumSkips"), DeviceIt->TotalTestSkips);
			DeviceArgs.Add(TEXT("NumFails"), NumTests - DeviceIt->TotalTestSuccesses - DeviceIt->TotalTestSkips);
			DeviceArgs.Add(TEXT("TotalTime"), DeviceIt->TotalTime);

			GridContainer->AddSlot(0,RowCounter)
				.HAlign(HAlign_Left)
				.Padding(0,0,3,0)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					[
						SNew(STextBlock)
						.Text(FText::FromString(DeviceIt->GameInstanceName))
					]
					+SVerticalBox::Slot()
					[
						SNew(STextBlock)
						.Text(FText::Format(LOCTEXT("AutomationGraphicalDeviceHeader", "{NumTests} Tests / {NumFails} Fails / {NumSkips} Skips / {TotalTime} Seconds"), DeviceArgs))
					]
				];

			TSharedRef<SHorizontalBox> TestContainer = SNew(SHorizontalBox);

			// If there are too many tests then result blocks may become invisible because they get too tiny
			// and margins are covering them up. Hence reducing the margins with relation to the number of tests.
			int32 TestsNum = DeviceIt->Tests.Num();
			float HorizontalMargin = TestsNum < 30 ? 1.0 : (TestsNum < 100 ? 0.5 : (TestsNum < 1000 ? 0.25 : 0.1));

			for( int32 TestIndex = 0; TestIndex < DeviceIt->Tests.Num(); ++TestIndex)
			{
				FTestResults* TestIt = &DeviceIt->Tests[TestIndex];

				FFormatNamedArguments TestArgs;	
				TestArgs.Add(TEXT("Duration"), TestIt->Duration);
				TestArgs.Add(TEXT("Name"), FText::FromString(TestIt->TestName));

				TestContainer->AddSlot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.Padding( FMargin(HorizontalMargin, 5) )
					.FillWidth(TestIt->Duration)
					[
						SNew(SOverlay)
						.ToolTipText(FText::Format(LOCTEXT("AutomationGraphicalToolTip", "{Name} \nDuration: {Duration}s"), TestArgs))

						+SOverlay::Slot()
						[
							SNew(SBorder)
							.BorderImage( FAppStyle::GetBrush("ErrorReporting.Box") )
							.BorderBackgroundColor(this,&SAutomationGraphicalResultBox::GetColorForTestState, TestIt->TestState, TestIt->bHasWarnings)
						]

						+ SOverlay::Slot()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(this,&SAutomationGraphicalResultBox::GetTestDisplayText,TestIt->TestName,TestIt->Duration)//FText::Format(LOCTEXT("AutomationGraphicalDuration", "{Duration}s"), TestArgs))
							.ColorAndOpacity(FLinearColor(0,0,0,1))
						]
					];
			}

			//Fill in the end with the remaining time
			if( DeviceIt->TotalTime < TotalTestDuration )
			{
				const float RemainingTime = TotalTestDuration - DeviceIt->TotalTime;
				TestContainer->AddSlot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.Padding( FMargin(1,5) )
					.FillWidth( RemainingTime )
					[
						SNew(SBorder)
						.BorderImage( FAppStyle::GetBrush("ErrorReporting.Box") )
						.BorderBackgroundColor(FLinearColor(0,0,0,0))
					];
			}

			RowCounter++;

			GridContainer->AddSlot(0,RowCounter)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding( FMargin(1,0) )
				.ColumnSpan(11)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(1.f)
					[
						TestContainer
					]
				];

			RowCounter++;
		}
	}

	RootBox->AddSlot()
		[
			GridContainer
		];
}


FSlateColor SAutomationGraphicalResultBox::GetColorForTestState(const EAutomationState TestState, const bool bHasWarnings) const
{
	switch(TestState)
	{
	case EAutomationState::Success:
		if (bHasWarnings)
		{
			return FSlateColor( FLinearColor( 1.0f, 0.5f, 0.0f ) );
		}
		else
		{
			return FSlateColor( FLinearColor( 0.0f, 0.5f, 0.0f ) );
		}
	case EAutomationState::Fail:
	default: //Default to fail!
		break;
	}

	return FSlateColor( FLinearColor( 0.5f, 0.0f, 0.0f ) );
}


FText SAutomationGraphicalResultBox::GetTestDisplayText(const FString TestName, const float TestTime) const
{
	if(DisplayType == EAutomationGrapicalDisplayType::DisplayName)
	{
		return FText::FromString(TestName);
	}
	else
	{
		FFormatNamedArguments TestArgs;	
		TestArgs.Add(TEXT("Duration"), TestTime);
		return FText::Format(LOCTEXT("AutomationGraphicalDuration", "{Duration}s"), TestArgs);
	}
}


EAutomationGrapicalDisplayType::Type SAutomationGraphicalResultBox::GetDisplayType() const
{
	return DisplayType;
}


void SAutomationGraphicalResultBox::SetDisplayType(EAutomationGrapicalDisplayType::Type NewDisplayType)
{
	DisplayType = NewDisplayType;
}


bool SAutomationGraphicalResultBox::HasResults() const
{
	return ClusterResults.Num() > 0;
}


#undef LOCTEXT_NAMESPACE
