// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "IAutomationControllerManager.h"
#include "Styling/SlateColor.h"
#include "Widgets/SCompoundWidget.h"

class SVerticalBox;

/**
 * The different supported items we can display on the test item
 */
namespace EAutomationGrapicalDisplayType
{
	enum Type
	{
		DisplayName,
		DisplayTime
	};
}


/**
 * Implements the automation graphical results box widget.
 */
class SAutomationGraphicalResultBox
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SAutomationGraphicalResultBox) { }

	SLATE_ARGUMENT(FString, Text)

	SLATE_END_ARGS()

	virtual ~SAutomationGraphicalResultBox();

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InAutomationController Reference to the automation controller.
	 */
	void Construct( const FArguments& InArgs, const IAutomationControllerManagerRef& InAutomationController);

	/** Clears the current results and any widgets that were created */
	void ClearResults();

	/** Returns if there are any results available to display. */
	bool HasResults() const;

	/** Returns the current display type. */
	EAutomationGrapicalDisplayType::Type GetDisplayType() const;

	/**
	 * Sets the current display type
	 *
	 * @param NewDisplayType The display type to set.
	 */
	void SetDisplayType(EAutomationGrapicalDisplayType::Type NewDisplayType);

private:

	/** Holds information on a single test result */
	class FTestResults
	{
	public:
		/** Constructor */
		FTestResults(const FString& InName)
		{
			TestName = InName;
			Duration = 0.f;
			TestState = EAutomationState::NotRun;
			bHasWarnings = false;
		}

		/** Name of the test */
		FString TestName;

		/** Duration of the test */
		float Duration;

		/** State of the test */
		EAutomationState TestState;

		/** If the test had any warnings */
		bool bHasWarnings;
	};

	/** Holds all the test results for a single device */
	class FDeviceResults
	{
	public:
		/** Constructor */
		FDeviceResults(const FGuid& InGameInstanceId, const FString& InGameInstanceName)
		{
			GameInstanceName = InGameInstanceName;
			GameInstanceId = InGameInstanceId.ToString();
			TotalTime = 0.f;
			TotalTestSkips = 0;
			TotalTestSuccesses = 0;
		}

		/** The game instance Name running on the Device */
		FString GameInstanceName;

		/** The game instance Id running on the Device */
		FString GameInstanceId;

		/** Total time of all the tests */
		float TotalTime;

		/** How many of the tests were skipped */
		uint32 TotalTestSkips;

		/** How many of the tests were successful */
		uint32 TotalTestSuccesses;

		/** The list of tests */
		TArray<FTestResults> Tests;
	};

	/** Holds all the results for a single cluster */
	class FClusterResults
	{
	public:
		/** Constructor */
		FClusterResults(const FString& InName)
		{
			ClusterName = InName;
			TotalTime = 0.f;
			ParallelTime = 0.f;
			TotalNumTests = 0;
			TotalTestSkips = 0;
			TotalTestSuccesses = 0;
		}

		/** Name of the Cluster */
		FString ClusterName;

		/** Total number of tests */
		uint32 TotalNumTests;

		/** How many of the tests were skipped */
		uint32 TotalTestSkips;

		/** How many of the tests were successful */
		uint32 TotalTestSuccesses;

		/** Total time of all the tests */
		float TotalTime;

		/** Longest time a device took to finish its tests */
		float ParallelTime;

		/** The list of devices in this cluster */
		TArray <FDeviceResults> Devices;
	};

	/**
	 * Returns if there are any results available to display.
	 *
	 * @param InReport The report to recursively check for enabled tests.
	 * @param OutReports The list of enabled test reports.
	 */
	void GetEnabledReports(TSharedPtr<IAutomationReport> InReport, TArray< TSharedPtr< IAutomationReport > >& OutReports);

	/** Populates the ClusterResults with test data pulled from the automation controller */
	void PopulateData();

	/** Creates all the slate widgets to display the test results */
	void CreateWidgets();

	/** This is called when the tests are complete so we can update the widget */
	void OnFillResults();

	/**
	 * Gets the color for a test based off the test state and if it has any warnings.
	 *
	 * @param TestState The test state from the automation report (Success, Fail, NotRun, etc.)
	 * @param bHasWarnings If the test had any warning logs.
	 */
	FSlateColor GetColorForTestState(const EAutomationState TestState, const bool bHasWarnings) const;

	/**
	 * Gets the text to display for this test item based off the current DisplayType.
	 *
	 * @param TestName The name of the test.
	 * @param TestTime The duration of the test.
	 */
	FText GetTestDisplayText(const FString TestName, const float TestTime) const;

	/** Stores what information should be displayed on the test widgets. */
	EAutomationGrapicalDisplayType::Type DisplayType;

	/** The test results grouped by cluster. */
	TArray<FClusterResults> ClusterResults;

	/** The time of the longest cluster. */
	float TotalTestDuration;

	/** Pointer to the root widget that contains all the result widgets. */
	TSharedPtr< SVerticalBox > RootBox;

	/** Pointer to the automation controller so we can get the test results. */
	IAutomationControllerManagerPtr AutomationController;
};
