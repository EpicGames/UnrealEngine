// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAutomationTestItem.h"
#include "Modules/ModuleManager.h"
#include "IAutomationReport.h"
#include "IAutomationControllerModule.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SSpinningImage.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSuggestionTextBox.h"
#include "SSimpleComboButton.h"
#include "AutomationWindowStyle.h"
#include "AutomationTestExcludelist.h"
#include "AutomationTestPlatform.h"
#include "SAutomationWindow.h"

#if WITH_EDITOR
	#include "AssetRegistry/AssetData.h"
	#include "EngineGlobals.h"
	#include "Editor.h"
	#include "AssetRegistry/AssetRegistryModule.h"
	#include "Dialogs/Dialogs.h"
	#include "SKismetInspector.h"
	#include "Internationalization/Regex.h"
#endif

#include "Widgets/Input/SHyperlink.h"
#include "SSimpleButton.h"

#include "SAutomationTestTagEditor.h"

#define LOCTEXT_NAMESPACE "AutomationTestItem"

namespace
{
#if WITH_EDITOR
	struct FTaskStringEntry
	{
	public:
		/** Stores found full ticket string including hashtag. */
		FString FullTicketString = {};
		/** Stores found ticket identifier string. */
		FString TicketIdString = {};
		/** Stores starting position (in original string) where FullTicketString starts. */
		int32 FullTicketBeginning = INDEX_NONE;
		/** Stores starting position (in original string) where TicketIdString starts. */
		int32 TicketIdBeginning = INDEX_NONE;

		/**
		 * Checks if the current instance stores valid data about task tracker's ticket.
		 *
		 * @return Return true if the instance is valid or false otherwise.
		 */
		bool IsValid() const
		{
			return !FullTicketString.IsEmpty() && (INDEX_NONE != FullTicketBeginning);
		}

		/**
		 * Parses the given string in order to find the last occurence of a task tracker's ticket.
		 * 
		 * @param Source The string to be parsed.
		 *
		 * @return Return valid FTaskStringEntry instance if the ticket information is found successfully or invalid object otherwise.
		 */
		static FTaskStringEntry LocateLastEntry(const FString& Source)
		{
			FTaskStringEntry ResultData;

			int32 LastEntryIndex = INDEX_NONE;
			{
				// First pass is to find the proper index of the last entry.
				FRegexMatcher TicketRegexMatcher(GetTaskTrackerTicketRegexPattern(), Source);

				while (TicketRegexMatcher.FindNext())
				{
					++LastEntryIndex;
				}
			}

			{
				// Second pass is to fill the ResultData
				FRegexMatcher TicketRegexMatcher(GetTaskTrackerTicketRegexPattern(), Source);

				for (int32 CurrentEntryIndex = 0; CurrentEntryIndex <= LastEntryIndex; ++CurrentEntryIndex)
				{
					TicketRegexMatcher.FindNext();
				}

				ResultData.FullTicketString = TicketRegexMatcher.GetCaptureGroup(0);
				ResultData.FullTicketBeginning = TicketRegexMatcher.GetCaptureGroupBeginning(0);
				// Note that the ticket id string might be empty
				ResultData.TicketIdString = TicketRegexMatcher.GetCaptureGroup(3);
				ResultData.TicketIdBeginning = TicketRegexMatcher.GetCaptureGroupBeginning(3);
			}

			return ResultData;
		}

	private:
		static FRegexPattern GetTaskTrackerTicketRegexPattern()
		{
			static const UAutomationTestExcludelist* Excludelist = UAutomationTestExcludelist::Get();
			check(nullptr != Excludelist);

			return FRegexPattern(FString::Format(
				TEXT("\\B({0})(\\s+([A-Za-z0-9\\-_]+))?\\b"),
				{
					Excludelist->GetTaskTrackerTicketTag()
				}));
		}
	};

	FString BuildTaskTrackerTicketURL(const FString& TaskTrackerURLBase, const FString& TaskTrackerTicketId)
	{
		check((!TaskTrackerURLBase.IsEmpty()) && (!TaskTrackerTicketId.IsEmpty()));

		const TMap<FString, FStringFormatArg> Args =
		{
			{ TEXT("ID"), TaskTrackerTicketId }
		};

		return FString::Format(*TaskTrackerURLBase, Args);
	}
#endif // WITH_EDITOR

} // anonymous namespace

/* SAutomationTestItem interface
 *****************************************************************************/

void SAutomationTestItem::Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
{
	TestStatus = InArgs._TestStatus;
	ColumnWidth = InArgs._ColumnWidth;
	HighlightText = InArgs._HighlightText;
	OnCheckedStateChangedDelegate = InArgs._OnCheckedStateChanged;
	IsLocalSession = InArgs._IsLocalSession;

	SMultiColumnTableRow< TSharedPtr< FString > >::Construct( SMultiColumnTableRow< TSharedPtr< FString > >::FArguments(), InOwnerTableView );
}


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SAutomationTestItem::GenerateWidgetForColumn( const FName& ColumnName )
{
	if (ColumnName == AutomationTestWindowConstants::Checked)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.Padding(4.0f, 0.0f)
			[
				//enabled/disabled check box
				SNew(SCheckBox)
				.IsChecked(this, &SAutomationTestItem::IsTestEnabled)
			.OnCheckStateChanged(this, &SAutomationTestItem::HandleTestingCheckbox_Click)
			];
	}
	else if (ColumnName == AutomationTestWindowConstants::Skipped)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAutomationWindowStyle::Get(), "SimpleButton")
				.ToolTipText(this, &SAutomationTestItem::GetExcludeReason)
				.IsEnabled(this, &SAutomationTestItem::CanSkipFlagBeChanged)
				.OnClicked(FOnClicked::CreateSP(this, &SAutomationTestItem::SetSkipFlag))
#if WITH_EDITOR
				.Cursor_Lambda([this]() {return this->IsLocalSession ? EMouseCursor::Hand : EMouseCursor::Default;})
				.OnHovered_Lambda([this]() {this->bIsToBeSkippedButtonHovered = true;})
				.OnUnhovered_Lambda([this]() {this->bIsToBeSkippedButtonHovered = false;})
#endif
				[
					SNew(SImage)
					.Image(FAutomationWindowStyle::Get().GetBrush("AutomationWindow.ExcludedTestsFilter"))
					.Visibility(this, &SAutomationTestItem::IsToBeSkipped_GetVisibility)
					.ColorAndOpacity(this, &SAutomationTestItem::IsToBeSkipped_GetColorAndOpacity)
				]
			];
	}
	else if (ColumnName == AutomationTestWindowConstants::SkippedOptions)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
#if WITH_EDITOR
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SNew(SSimpleButton)
				.Icon(FAutomationWindowStyle::Get().GetBrush("Icons.Edit"))
				.Cursor_Lambda([this]() {return this->IsLocalSession ? EMouseCursor::Hand : EMouseCursor::Default;})
				.Visibility(this, &SAutomationTestItem::IsDirectlyExcluded_GetVisibility)
				.ToolTipText(LOCTEXT("EditExcludeOptions", "Edit exclude options"))
				.OnClicked(FOnClicked::CreateSP(this, &SAutomationTestItem::OnEditExcludeOptionsClicked))
			]
#endif
			;
	}
	else if (ColumnName == AutomationTestWindowConstants::Tags)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SSimpleButton)
					.Icon(FAutomationWindowStyle::Get().GetBrush("Icons.Edit"))
					.Visibility(this, &SAutomationTestItem::CanEditTags_GetVisibility)
					.ToolTipText(LOCTEXT("EditTestTags", "Edit Tags"))
					.OnClicked(FOnClicked::CreateSP(this, &SAutomationTestItem::OnEditTestTagsClicked))
			]
			+ SHorizontalBox::Slot()
			.Expose(TagsSlot)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
					.Margin(FMargin(2, 0, 0, 0))
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					.Text(FText::FromString(TestStatus->GetTags()))
					.ToolTipText(FText::FromString(TestStatus->GetTags()))
			];
	}
	else if (ColumnName == AutomationTestWindowConstants::Title)
	{
		TSharedRef<SWidget> TestNameWidget = SNullWidget::NullWidget;

		// Would be nice to warp to text location...more difficult when distributed.
		if ( !TestStatus->GetOpenCommand().IsEmpty() && WITH_EDITOR )
		{
#if WITH_EDITOR
			TestNameWidget = SNew(SHyperlink)
				.Style(FAutomationWindowStyle::Get(), "Common.GotoNativeCodeHyperlink")
				.OnNavigate_Lambda([this] {
					GEngine->Exec(nullptr, *TestStatus->GetOpenCommand());
				})
				.HighlightText(HighlightText)
				.Text(FText::FromString(TestStatus->GetDisplayNameWithDecoration()));
#endif
		}
		else if ( !TestStatus->GetAssetPath().IsEmpty() && WITH_EDITOR )
		{
#if WITH_EDITOR
			TestNameWidget = SNew(SHyperlink)
				.Style(FAutomationWindowStyle::Get(), "Common.GotoNativeCodeHyperlink")
				.OnNavigate_Lambda([this] {
					FString AssetPath = TestStatus->GetAssetPath();
					FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

					TArray<FAssetData> AllAssets;
					AssetRegistryModule.Get().GetAssetsByPackageName(*AssetPath, AllAssets);

					if ( AllAssets.Num() > 0 )
					{
						UObject* ObjectToEdit = AllAssets[0].GetAsset();
						if ( ObjectToEdit )
						{
							GEditor->EditObject(ObjectToEdit);
						}
					}
				})
				.HighlightText(HighlightText)
				.Text(FText::FromString(TestStatus->GetDisplayNameWithDecoration()));
#endif
		}
		else if ( !TestStatus->GetSourceFile().IsEmpty() )
		{
			TestNameWidget = SNew(SHyperlink)
				.Style(FAutomationWindowStyle::Get(), "Common.GotoNativeCodeHyperlink")
				.OnNavigate_Lambda([this] { FSlateApplication::Get().GotoLineInSource(TestStatus->GetSourceFile(), TestStatus->GetSourceFileLine()); })
				.HighlightText(HighlightText)
				.Text(FText::FromString(TestStatus->GetDisplayNameWithDecoration()));
		}
		else
		{
			TestNameWidget = SNew(STextBlock)
				.HighlightText(HighlightText)
				.Text(FText::FromString(TestStatus->GetDisplayNameWithDecoration()));
		}

		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				//this is where the tree is marked as expandable or not.
				SNew( SExpanderArrow, SharedThis( this ) )
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			[
				//name of the test
				TestNameWidget
			];
	}
	else if( ColumnName == AutomationTestWindowConstants::SmokeTest )
	{
		//icon to show if the test is considered fast or is the parent of a fast test
		return SNew( SImage) 
			.Image( this, &SAutomationTestItem::GetSmokeTestImage );
	}
	else if( ColumnName == AutomationTestWindowConstants::RequiredDeviceCount )
	{
		// Should we display an icon to indicate that this test "Requires" more than one participant?
		if( TestStatus->GetNumParticipantsRequired() > 1 )
		{
			TSharedPtr< SHorizontalBox > HBox = SNew( SHorizontalBox );
			if( TestStatus->GetTotalNumChildren() == 0 )
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("NumParticipantsRequired"), TestStatus->GetNumParticipantsRequired());

				// Display a network PC and the number which are required for this test.
				HBox->AddSlot()
					[
						SNew( SImage )
						.Image(FAutomationWindowStyle::Get().GetBrush("Automation.Participant") )
					];
				HBox->AddSlot()
					[
						SNew( STextBlock )
						.Text( FText::Format( LOCTEXT( "NumParticipantsRequiredWrapper", "x{NumParticipantsRequired}" ), Args ) )
					];

				HBox->SetToolTipText(FText::Format(LOCTEXT("NumParticipantsRequiredMessage", "This test requires {NumParticipantsRequired} participants to be run."), Args));

			}
			else
			{
				HBox->AddSlot()
					.HAlign(HAlign_Center)
					[
						SNew( SImage )
						.Image(FAutomationWindowStyle::Get().GetBrush("Automation.ParticipantsWarning") )
						.ToolTipText( LOCTEXT("ParticipantsWarningToolTip", "Some tests require multiple participants") )
					];
			}
			return HBox.ToSharedRef();
		}
	}
	else if( ColumnName == AutomationTestWindowConstants::Status )
	{
		TSharedRef<SHorizontalBox> HBox = SNew (SHorizontalBox);
		int32 NumClusters = FModuleManager::GetModuleChecked<IAutomationControllerModule>("AutomationController").GetAutomationController()->GetNumDeviceClusters();

		//for each cluster, display a status icon
		for (int32 ClusterIndex = 0; ClusterIndex < NumClusters; ++ClusterIndex)
		{
			//if this is a leaf test
			if (TestStatus->GetTotalNumChildren() == 0)
			{
				//for leaf tests
				HBox->AddSlot()
				.MaxWidth(ColumnWidth)
				.FillWidth(1.0)
				[			
					SNew(SBorder)
					.BorderImage(FAutomationWindowStyle::Get().GetBrush("ErrorReporting.Box") )
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding( FMargin(3,0) )
					.BorderBackgroundColor( FSlateColor( FLinearColor( 1.0f, 0.0f, 1.0f, 0.0f ) ) )
					.ToolTipText( this, &SAutomationTestItem::GetTestToolTip, ClusterIndex )
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							//image when complete or un-run
							SNew( SImage )
							.Image( this, &SAutomationTestItem::ItemStatus_StatusImage, ClusterIndex )
							.Visibility( this, &SAutomationTestItem::ItemStatus_GetStatusVisibility, ClusterIndex, false )
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(SBox)
							.WidthOverride(16.0f)
							.HeightOverride(16.0f)
							[
								// Spinning Image while in process
								SNew(SSpinningImage)
								.Image( this, &SAutomationTestItem::ItemStatus_StatusImage, ClusterIndex )
								.Visibility(this, &SAutomationTestItem::ItemStatus_GetStatusVisibility, ClusterIndex, true)
							]
						]
					]
				];
			}
			else
			{
				//for internal tree nodes
				HBox->AddSlot()
				.MaxWidth(ColumnWidth)
				.FillWidth(1.0)
				[
					SNew(SBorder)
					.BorderImage(FAutomationWindowStyle::Get().GetBrush("ErrorReporting.Box") )
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding( FMargin(3,0) )
					.BorderBackgroundColor( FSlateColor( FLinearColor( 1.0f, 0.0f, 1.0f, 0.0f ) ) )
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							//image when children complete or not run
							SNew(SImage)
							.Image(this, &SAutomationTestItem::ItemChildrenStatus_StatusImage, ClusterIndex)
							.Visibility(this, &SAutomationTestItem::ItemStatus_GetChildrenStatusVisibility, ClusterIndex, false)
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SBox)
							.WidthOverride(ColumnWidth - 8)
							.HeightOverride(16.0f)
							[
								//progress bar for percent of enabled children completed
								SNew(SProgressBar)
								.Percent(this, &SAutomationTestItem::ItemStatus_ProgressFraction, ClusterIndex)
								.FillColorAndOpacity(this, &SAutomationTestItem::ItemStatus_ProgressColor, ClusterIndex)
								.Visibility(this, &SAutomationTestItem::ItemStatus_GetChildrenStatusVisibility, ClusterIndex, true)
							]
						]
					]
				];
			}
		}
		return HBox;
	}
	else if( ColumnName == AutomationTestWindowConstants::Timing )
	{
		return SNew( STextBlock )
		.Text( this, &SAutomationTestItem::ItemStatus_DurationText);
	}


	return SNullWidget::NullWidget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SAutomationTestItem Implementation
 *****************************************************************************/
const FSlateBrush* SAutomationTestItem::GetSmokeTestImage() const
{
	const FSlateBrush* ImageToUse = nullptr;
	if (!!(TestStatus->GetTestFlags() & EAutomationTestFlags::SmokeFilter))
	{
		if ( TestStatus->IsParent() )
		{
			ImageToUse = FAutomationWindowStyle::Get().GetBrush("Automation.SmokeTest");
		}
		else
		{
			ImageToUse = FAutomationWindowStyle::Get().GetBrush("Automation.SmokeTestParent");
		}
	}

	return ImageToUse;
}


FText SAutomationTestItem::GetTestToolTip( int32 ClusterIndex ) const
{
	FText TestToolTip;
	const int32 PassIndex = TestStatus->GetCurrentPassIndex(ClusterIndex);
	EAutomationState TestState = TestStatus->GetState( ClusterIndex, PassIndex );
	if ( TestState == EAutomationState::NotRun )
	{
		TestToolTip = LOCTEXT("TestToolTipNotRun", "Not Run");
	}
	else if( TestState == EAutomationState::Skipped )
	{
		TestToolTip = LOCTEXT("ToolTipSkipped", "This test was skipped.");
	}
	else
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("GameInstance"), FText::FromString(TestStatus->GetGameInstanceName(ClusterIndex)));

		if (TestState == EAutomationState::InProcess)
		{
			TestToolTip = FText::Format(LOCTEXT("TestToolTipInProgress", "In progress on: {GameInstance}"), Args);
		}
		else if (TestState == EAutomationState::Success)
		{
			TestToolTip = FText::Format(LOCTEXT("TestToolTipComplete", "Completed on: {GameInstance}"), Args);
		}
		else
		{
			TestToolTip = FText::Format(LOCTEXT("TestToolTipFailed", "Failed on: {GameInstance}"), Args);
		}
	}
	return TestToolTip;
}


ECheckBoxState SAutomationTestItem::IsTestEnabled() const
{
	return TestStatus->IsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

EVisibility SAutomationTestItem::IsToBeSkipped_GetVisibility() const
{
	if (bIsToBeSkippedButtonHovered && IsLocalSession)
	{
		return EVisibility::Visible;
	}
	return TestStatus->IsToBeSkipped() ? EVisibility::Visible : EVisibility::Hidden;
}

FSlateColor SAutomationTestItem::IsToBeSkipped_GetColorAndOpacity() const
{
	if (bIsToBeSkippedButtonHovered && !TestStatus->IsToBeSkipped())
	{
		return FLinearColor(1.0f, 1.0f, 1.0f, 0.4f);
	}

	// Identify visually if the test is to be skipped on certain condition.
	if (TestStatus->IsToBeSkippedOnConditions())
	{
		return FLinearColor(1.0f, 1.0f, 0.2f, 0.6f);
	}

	return FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
}

bool SAutomationTestItem::IsDirectlyExcluded() const
{
	return WITH_EDITOR && IsLocalSession && TestStatus->IsToBeSkipped() && !TestStatus->IsToBeSkippedByPropagation();
}

EVisibility SAutomationTestItem::IsDirectlyExcluded_GetVisibility() const
{
	return  (WITH_EDITOR && IsLocalSession && (IsDirectlyExcluded() || TestStatus->IsToBeSkippedOnConditions())) ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SAutomationTestItem::GetExcludeReason() const
{
	FName Reason;
	bool IsToBeSkipped = TestStatus->IsToBeSkipped(&Reason);

	if (IsToBeSkipped)
	{
		return FText::FromName(Reason);
	}

	return IsLocalSession ? LOCTEXT("ExludeTest", "Exclude test") : FText();
}

FReply SAutomationTestItem::SetSkipFlag()
{
#if WITH_EDITOR
	// If it's not local session editing is disabled
	if (IsLocalSession == false)
	{
		return FReply::Handled();
	}
	if (!TestStatus->IsToBeSkipped())
	{
		OnEditExcludeOptionsClicked();
	}
	else
	{
		TestStatus->SetSkipFlag(false);
	}
#endif
	return FReply::Handled();
}

bool SAutomationTestItem::CanSkipFlagBeChanged() const
{
	return WITH_EDITOR && IsLocalSession && !TestStatus->IsToBeSkippedByPropagation();
}

EVisibility SAutomationTestItem::CanEditTags_GetVisibility() const
{
	return (WITH_EDITOR && !TestStatus->IsParent()) ? EVisibility::Visible : EVisibility::Collapsed;
}

#if WITH_EDITOR
void PopulateMenuContent(FMenuBuilder* MenuBuilder, TSet<FName>* OptionsDestination, const TSet<FName>& ItemNames)
{
	for (const FName& Item : ItemNames)
	{
		TSharedRef<SWidget> FlagWidget =
			SNew(SCheckBox)
			.IsChecked(OptionsDestination->Contains(Item))
			.OnCheckStateChanged_Lambda([OptionsDestination, Item](ECheckBoxState NewState)
				{
					if (NewState == ECheckBoxState::Checked)
					{
						OptionsDestination->Add(Item);
					}
					else
					{
						OptionsDestination->Remove(Item);
					}
				})
			.Padding(FMargin(4.0f, 0.0f))
					.Content()
					[
						SNew(STextBlock)
						.Text(FText::FromName(Item))
					];

		MenuBuilder->AddWidget(FlagWidget, FText::GetEmpty());
	}
}

TSharedRef<SWidget> GeneratePlatformMenuContentFromExcludeOptions(TSharedPtr<FAutomationTestExcludeOptions> Options)
{
	static const TSet<FName>& AllPlatform_OptionNames = AutomationTestPlatform::GetAllAvailablePlatformNames();
	FMenuBuilder MenuBuilder(false, nullptr);
	PopulateMenuContent(&MenuBuilder, &Options->Platforms, AllPlatform_OptionNames);

	return MenuBuilder.MakeWidget();
}

FText GeneratePlatformTextFromExcludeOptions(TSharedPtr<FAutomationTestExcludeOptions> Options)
{
	if (Options->Platforms.Num() == 0)
	{
		return LOCTEXT("ExcludeOptions_Platform_All", "All Platforms");
	}

	return FText::FromString(SetToString(Options->Platforms));
}

TSharedRef<SWidget> GenerateRHIMenuContentFromExcludeOptions(TSharedPtr<FAutomationTestExcludeOptions> Options)
{
	FMenuBuilder MenuBuilder(false, nullptr);

	static const TSet<FName>& AllRHI_OptionNames = FAutomationTestExcludeOptions::GetAllRHIOptionNamesFromSettings();
	MenuBuilder.BeginSection("AutomationWindow_ExcludeOptions_RHI", LOCTEXT("ExcludeOptions_RHI_Section", "Interfaces"));
	PopulateMenuContent(&MenuBuilder, &Options->RHIs, AllRHI_OptionNames);
	MenuBuilder.EndSection();

	static const TSet<FName>& AllRHI_FeatureLevel_OptionNames = FAutomationTestExcludeOptions::GetAllRHIOptionNames<ETEST_RHI_FeatureLevel_Options>();
	MenuBuilder.BeginSection("AutomationWindow_ExcludeOptions_RHI_FeatureLevel", LOCTEXT("ExcludeOptions_RHI_FeatureLevel_Section", "Feature Levels"));
	PopulateMenuContent(&MenuBuilder, &Options->RHIs, AllRHI_FeatureLevel_OptionNames);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FText GenerateRHITextFromExcludeOptions(TSharedPtr<FAutomationTestExcludeOptions> Options)
{
	if (Options->RHIs.Num() == 0)
	{
		return LOCTEXT("ExcludeOptions_RHI_All", "All Interfaces");
	}

	return FText::FromString(SetToString(Options->RHIs));
}
#endif

FReply SAutomationTestItem::OnEditExcludeOptionsClicked()
{
#if WITH_EDITOR
	TSharedPtr<FAutomationTestExcludeOptions> Options = TestStatus->GetExcludeOptions();
	check(Options.IsValid());

	static const UAutomationTestExcludelist* Excludelist = UAutomationTestExcludelist::Get();
	check(nullptr != Excludelist);
	const FString TaskTrackerURLBase = Excludelist->GetTaskTrackerURLBase();
	const FString TaskTrackerURLHashtag = Excludelist->GetConfigTaskTrackerHashtag();
	
	const bool TaskTrackerSlotIsVisible =
		!(TaskTrackerURLBase.IsEmpty() && TaskTrackerURLHashtag.IsEmpty());
	const bool OpenHyperlinkButtonIsEnabled = 
		(TaskTrackerSlotIsVisible && (!TaskTrackerURLBase.IsEmpty()));

	FString BeautifiedReason = Options->Reason.ToString();
	FString TaskTrackerTicketId;

	if (TaskTrackerSlotIsVisible)
	{
		FTaskStringEntry TaskTrackerTicketEntry = FTaskStringEntry::LocateLastEntry(BeautifiedReason);

		if (TaskTrackerTicketEntry.IsValid() && !TaskTrackerTicketEntry.TicketIdString.IsEmpty())
		{
			TaskTrackerTicketId = TaskTrackerTicketEntry.TicketIdString;
			BeautifiedReason.RemoveAt(TaskTrackerTicketEntry.FullTicketBeginning, TaskTrackerTicketEntry.FullTicketString.Len());
		}
	}

	// Define the dialog form.
	TSharedPtr<SVerticalBox> VBox;
	TSharedRef<SWidget> Form = SNew(SBox)
		.WidthOverride(350)
		[
			SAssignNew(VBox, SVerticalBox)
		];

	VBox->AddSlot().AutoHeight().Padding(5.0f)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.MaxWidth(55)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(5, 0)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("ExcludeOptions_TestLabel", "Test"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(3)
				[
					SNew(SEditableTextBox)
						.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
						.Text(FText::FromName(Options->Test))
						.ToolTipText(FText::FromName(Options->Test))
						.IsReadOnly(true)
				]
		];

	VBox->AddSlot().AutoHeight().Padding(5.0f)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.MaxWidth(55)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(5, 0)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("ExcludeOptions_Reason", "Reason"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(3)
				[
					SNew(SEditableTextBox)
						.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
						.Text(FText::FromString(BeautifiedReason))
						.OnTextCommitted_Lambda([Options, &BeautifiedReason, &TaskTrackerTicketId](const FText& NewBeautifiedReason, const ETextCommit::Type&)
						{
							BeautifiedReason = NewBeautifiedReason.ToString();
							Options->UpdateReason(BeautifiedReason, TaskTrackerTicketId);
						})
						.ToolTipText(LOCTEXT("ExcludeOptions_Reason_ToolTip", "The reason as to why the test is excluded"))
				]
		];

	if (TaskTrackerSlotIsVisible)
	{
		VBox->AddSlot().AutoHeight().Padding(5.0f)
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.MaxWidth(55)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(5, 0)
					[
						SNew(STextBlock)
							.Text(FText::FromString(Excludelist->GetTaskTrackerName()))
					]
					+ SHorizontalBox::Slot()
					.FillWidth(3.5)
					[
						SNew(SEditableTextBox)
							.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
							.Text(FText::FromString(TaskTrackerTicketId))
							.ToolTipText(LOCTEXT("ExcludeOptions_TaskTracker_ToolTip", "Task identifier related to the skipping reason"))
							.OnTextCommitted_Lambda([Options, &BeautifiedReason, &TaskTrackerTicketId](const FText& NewTaskTrackerTicketId, const ETextCommit::Type& CommitType)
							{
								TaskTrackerTicketId = NewTaskTrackerTicketId.ToString();
								Options->UpdateReason(BeautifiedReason, TaskTrackerTicketId);
							})
					]
					+ SHorizontalBox::Slot()
					.Padding(5, 0)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(SHyperlink)
							.Style(FAutomationWindowStyle::Get(), "Common.GotoNativeCodeHyperlink")
							.IsEnabled_Lambda([OpenHyperlinkButtonIsEnabled, &TaskTrackerTicketId]() { return OpenHyperlinkButtonIsEnabled && (!TaskTrackerTicketId.IsEmpty()); })
							.OnNavigate_Lambda([&TaskTrackerURLBase, &TaskTrackerTicketId]() { FPlatformProcess::LaunchURL(*(BuildTaskTrackerTicketURL(TaskTrackerURLBase, TaskTrackerTicketId)), nullptr, nullptr); })
							.Text(FText::FromString(TEXT("Open")))
					]
			];
	}

	VBox->AddSlot().AutoHeight().Padding(5.0f)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.MaxWidth(55)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(5, 0)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("ExcludeOptions_Platform", "Platforms"))
				]
				+ SHorizontalBox::Slot()
				.MaxWidth(200)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SComboButton)
						.OnGetMenuContent_Lambda([Options]() { return GeneratePlatformMenuContentFromExcludeOptions(Options); })
						.HasDownArrow(true)
						.ButtonContent()
						[
							SNew(STextBlock)
								.Text_Lambda([Options]() { return GeneratePlatformTextFromExcludeOptions(Options); })
								.ToolTipText_Lambda([Options]() { return GeneratePlatformTextFromExcludeOptions(Options); })
								.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
						]
				]
		];

	VBox->AddSlot().AutoHeight().Padding(5.0f)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.MaxWidth(55)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(5, 0)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("ExcludeOptions_RHI", "RHIs"))
				]
				+ SHorizontalBox::Slot()
				.MaxWidth(200)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SComboButton)
						.OnGetMenuContent_Lambda([Options]() { return GenerateRHIMenuContentFromExcludeOptions(Options); })
						.HasDownArrow(true)
						.ButtonContent()
						[
							SNew(STextBlock)
								.Text_Lambda([Options]() { return GenerateRHITextFromExcludeOptions(Options); })
								.ToolTipText_Lambda([Options]() { return GenerateRHITextFromExcludeOptions(Options); })
								.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
						]
				]
		];

	VBox->AddSlot().AutoHeight().Padding(5.0f)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.MaxWidth(55)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(5, 0)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("ExcludeOptions_Warn", "Warn"))
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SCheckBox)
						.ToolTipText(LOCTEXT("ExcludeOptions_Warn_ToolTip", "Raise a warning when skipping this test"))
						.IsChecked(Options->Warn)
						.OnCheckStateChanged_Lambda([this, Options](ECheckBoxState NewState) { Options->Warn = NewState == ECheckBoxState::Checked; })
				]
		];

	SGenericDialogWidget::FArguments DialogArguments;
	DialogArguments.OnOkPressed_Lambda([Options, this]()
		{
			auto Entry = FAutomationTestExcludelistEntry(*Options);
			TestStatus->SetSkipFlag(true, &Entry, false);
		});

	SGenericDialogWidget::OpenDialog(LOCTEXT("ExcludeTestOptions", "Exclude Test Options"), Form, DialogArguments, true);
#endif // WITH_EDITOR
	
	return FReply::Handled();
}

FReply SAutomationTestItem::OnEditTestTagsClicked()
{
#if WITH_EDITOR
	TArray<IAutomationReportPtr> ReportList{ TestStatus };
	SAutomationTestTagEditor::ShowInWindow(this->AsShared(), ReportList,
		FOnTestTagsSavedEvent::CreateLambda([this]()
		{
			FChildren* ContainerWidgetChildren = TagsSlot->GetContainerWidget().GetChildren();
			if (ContainerWidgetChildren && ContainerWidgetChildren->Num() > 1)
			{
				STextBlock& TagsText = static_cast<STextBlock&>(ContainerWidgetChildren->GetChildAt(1).Get());
				FString UpdatedTags = TestStatus->GetTags();
				TagsText.SetText(FText::FromString(UpdatedTags));
				TagsText.SetToolTipText(FText::FromString(UpdatedTags));
			}
		}));
#endif
	return FReply::Handled();
}

FSlateColor SAutomationTestItem::ItemStatus_BackgroundColor(const int32 ClusterIndex) const
{
	if (TestStatus->GetTotalNumChildren()==0)
	{
		const int32 PassIndex = TestStatus->GetCurrentPassIndex(ClusterIndex);
		EAutomationState TestState = TestStatus->GetState(ClusterIndex,PassIndex);
		if (TestState == EAutomationState::Fail)
		{
			// Failure is marked by a red background.
			return FSlateColor( FLinearColor( 0.5f, 0.0f, 0.0f ) );
		}
		else if (TestState == EAutomationState::InProcess)
		{
			// In Process, yellow.
			return FSlateColor( FLinearColor( 0.5f, 0.5f, 0.0f ) );
		}
		else if (TestState == EAutomationState::Success)
		{
			// Success is marked by a green background.
			return FSlateColor( FLinearColor( 0.0f, 0.5f, 0.0f ) );
		}

		// Not Scheduled will receive this color which is to say no color since alpha is 0.
		return FSlateColor( FLinearColor( 1.0f, 0.0f, 1.0f, 0.0f ) );
	}
	else
	{
		// Not Scheduled will receive this color which is to say no color since alpha is 0.
		return FSlateColor( FLinearColor( 1.0f, 0.0f, 1.0f, 0.0f ) );
	}
}


FText SAutomationTestItem::ItemStatus_DurationText() const
{
	FText DurationText;
	float MinDuration;
	float MaxDuration;
	if (TestStatus->GetDurationRange(MinDuration, MaxDuration))
	{
		FNumberFormattingOptions Options;
		Options.MaximumFractionalDigits = 4;
		Options.MaximumIntegralDigits = 4;

		FFormatNamedArguments Args;
		Args.Add(TEXT("MinDuration"), MinDuration);
		Args.Add(TEXT("MaxDuration"), MaxDuration);

		//if there is a duration range
		if (MinDuration != MaxDuration)
		{
			DurationText = FText::Format(LOCTEXT("ItemStatusDurationRange", "{MinDuration}s - {MaxDuration}s"), Args);
		}
		else
		{
			DurationText = FText::Format(LOCTEXT("ItemStatusDuration", "{MinDuration}s"), Args);
		}
	}

	return DurationText;
}


EVisibility SAutomationTestItem::ItemStatus_GetStatusVisibility(const int32 ClusterIndex, const bool bForInProcessThrobber) const
{
	const int32 PassIndex = TestStatus->GetCurrentPassIndex(ClusterIndex);
	EAutomationState TestState = TestStatus->GetState(ClusterIndex,PassIndex);
	bool bImageVisible = TestState != EAutomationState::InProcess;

	bool bFinalVisibility =  bForInProcessThrobber ? !bImageVisible : bImageVisible;

	return bFinalVisibility ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SAutomationTestItem::ItemStatus_GetChildrenStatusVisibility(const int32 ClusterIndex, const bool bForInProcessThrobber) const
{
	// Internal node: completion status image visible when all children completed
	bool bImageVisible = false;

	FAutomationCompleteState CompleteState;
	const int32 PassIndex = TestStatus->GetCurrentPassIndex(ClusterIndex);
	TestStatus->GetCompletionStatus(ClusterIndex, PassIndex, CompleteState);

	uint32 TotalComplete = CompleteState.NumEnabledTestsPassed + CompleteState.NumEnabledTestsFailed + CompleteState.NumEnabledTestsCouldntBeRun;
	if ((TotalComplete > 0) && (CompleteState.TotalEnabled > 0))
	{
		bImageVisible = (TotalComplete == CompleteState.TotalEnabled);
	}

	bool bFinalVisibility = bForInProcessThrobber ? !bImageVisible : bImageVisible;

	return bFinalVisibility ? EVisibility::Visible : EVisibility::Collapsed;
}


FText SAutomationTestItem::ItemStatus_NumParticipantsRequiredText() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("NumParticipantsRequired"), TestStatus->GetNumParticipantsRequired());

	return FText::Format(LOCTEXT("NumParticipantsRequiredWrapper", "x{NumParticipantsRequired}"), Args);
}


FSlateColor SAutomationTestItem::ItemStatus_ProgressColor(const int32 ClusterIndex) const
{
	FAutomationCompleteState CompleteState;
	const int32 PassIndex = TestStatus->GetCurrentPassIndex(ClusterIndex);
	TestStatus->GetCompletionStatus(ClusterIndex,PassIndex, CompleteState);

	if (CompleteState.TotalEnabled > 0)
	{
		if (CompleteState.NumEnabledTestsFailed > 0)
		{
			// Failure is marked by a red background.
			return FSlateColor( FLinearColor( 1.0f, 0.0f, 0.0f ) );
		}
		else if((CompleteState.NumEnabledTestsPassed !=  CompleteState.TotalEnabled) || 
			(CompleteState.NumEnabledTestsWarnings > 0) ||
			(CompleteState.NumEnabledTestsCouldntBeRun > 0 ))
		{
			// In Process, yellow.
			return FSlateColor( FLinearColor( 1.0f, 1.0f, 0.0f ) );
		}
		else
		{
			// Success is marked by a green background.
			return FSlateColor( FLinearColor( 0.0f, 1.0f, 0.0f ) );
		}
	}

	// Not Scheduled will receive this color which is to say no color since alpha is 0.
	return FSlateColor( FLinearColor( 1.0f, 0.0f, 1.0f, 0.0f ) );
}


TOptional<float> SAutomationTestItem::ItemStatus_ProgressFraction(const int32 ClusterIndex) const
{
	FAutomationCompleteState CompleteState;
	const int32 PassIndex = TestStatus->GetCurrentPassIndex(ClusterIndex);
	TestStatus->GetCompletionStatus(ClusterIndex, PassIndex, CompleteState);

	uint32 TotalComplete = CompleteState.NumEnabledTestsPassed + CompleteState.NumEnabledTestsFailed + CompleteState.NumEnabledTestsCouldntBeRun;
	// Only show a percentage if there is something interesting to report
	if( (TotalComplete> 0) && (CompleteState.TotalEnabled > 0) )
	{
		return (float)TotalComplete/CompleteState.TotalEnabled;
	}
	// Return incomplete state
	return 0.0f;
}


const FSlateBrush* SAutomationTestItem::ItemStatus_StatusImage(const int32 ClusterIndex) const
{
	const int32 PassIndex = TestStatus->GetCurrentPassIndex(ClusterIndex);
	EAutomationState TestState = TestStatus->GetState(ClusterIndex,PassIndex);

	const FSlateBrush* ImageToUse;
	switch( TestState )
	{
	case EAutomationState::Success:
		{
			FAutomationCompleteState CompleteState;
			TestStatus->GetCompletionStatus(ClusterIndex,PassIndex, CompleteState);
			//If there were ANY warnings in the results
			if (CompleteState.NumEnabledTestsWarnings || CompleteState.NumDisabledTestsWarnings)
			{
				ImageToUse = FAutomationWindowStyle::Get().GetBrush("Automation.Warning");
			}
			else
			{
				ImageToUse = FAutomationWindowStyle::Get().GetBrush("Automation.Success");
			}
		}
		break;

	case EAutomationState::Fail:
		ImageToUse = FAutomationWindowStyle::Get().GetBrush("Automation.Fail");
		break;

	case EAutomationState::NotRun:
		{
			ImageToUse = FAutomationWindowStyle::Get().GetBrush("Automation.NotRun");
		}
		break;

	case EAutomationState::Skipped:
		ImageToUse = FAutomationWindowStyle::Get().GetBrush("Automation.Skipped");
		break;

	default:
	case EAutomationState::InProcess:
		ImageToUse = FAutomationWindowStyle::Get().GetBrush("Automation.InProcess");
		break;
	}

	return ImageToUse;
}

const FSlateBrush* SAutomationTestItem::ItemChildrenStatus_StatusImage(const int32 ClusterIndex) const
{
	FAutomationCompleteState CompleteState;
	const int32 PassIndex = TestStatus->GetCurrentPassIndex(ClusterIndex);
	TestStatus->GetCompletionStatus(ClusterIndex, PassIndex, CompleteState);

	const FSlateBrush* ImageToUse = FAutomationWindowStyle::Get().GetBrush("Automation.InProcess");

	uint32 TotalComplete = CompleteState.NumEnabledTestsPassed + CompleteState.NumEnabledTestsFailed + CompleteState.NumEnabledTestsCouldntBeRun;
	if ((TotalComplete > 0) && (CompleteState.TotalEnabled > 0) && TotalComplete == CompleteState.TotalEnabled) {
		if (TotalComplete == CompleteState.NumEnabledTestsPassed)
		{
			// If there were ANY warnings in the results
			if (CompleteState.NumEnabledTestsWarnings > 0 || CompleteState.NumDisabledTestsWarnings > 0)
			{
				ImageToUse = FAutomationWindowStyle::Get().GetBrush("Automation.Warning");
			}
			else
			{
				ImageToUse = FAutomationWindowStyle::Get().GetBrush("Automation.Success");
			}
		}
		else if (CompleteState.NumEnabledTestsFailed)
		{
			ImageToUse = FAutomationWindowStyle::Get().GetBrush("Automation.Fail");
		}
		else if (CompleteState.NumEnabledTestsCouldntBeRun)
		{
			ImageToUse = FAutomationWindowStyle::Get().GetBrush("Automation.NotRun");
		}
	}
	return ImageToUse;
}

/* SAutomationTestitem event handlers
 *****************************************************************************/

void SAutomationTestItem::HandleTestingCheckbox_Click(ECheckBoxState)
{
	OnCheckedStateChangedDelegate.ExecuteIfBound(TestStatus);
}


#undef LOCTEXT_NAMESPACE
