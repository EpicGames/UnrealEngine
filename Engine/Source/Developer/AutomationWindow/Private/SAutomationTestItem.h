// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"


class IAutomationReport;

DECLARE_DELEGATE_OneParam(FOnItemCheckedStateChanged, TSharedPtr<IAutomationReport> /*TestStatus*/);

/**
 * Implements a row widget for the automation list.
 */
class SAutomationTestItem : public SMultiColumnTableRow< TSharedPtr<FString> >
{
public:	
	SLATE_BEGIN_ARGS(SAutomationTestItem)
		{}
		SLATE_ARGUMENT(float, ColumnWidth)
		SLATE_ARGUMENT(bool, IsLocalSession)
		SLATE_ARGUMENT(TSharedPtr<IAutomationReport>, TestStatus)
		SLATE_ATTRIBUTE(FText, HighlightText)
		/** Delegate called when a report has it's checkbox clicked */
		SLATE_EVENT(FOnItemCheckedStateChanged, OnCheckedStateChanged)
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs	          Declaration used by the SNew() macro to construct this widget.
	 * @param   InOwnerTableView  The owner table into which this row is being placed.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView );

	/**
	 * Generates the widget for the specified column.
	 *
	 * @param ColumnName The name of the column to generate the widget for.
	 * @return The widget.
	 */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override;

protected:

	/**
	 * Returns filter status color.
	 *
	 * @return filter color.
	 */
	FSlateColor GetFilterStatusTextColor( ) const;

	/**
	 * Returns icon for "fast" tests, parents of fast tests, or empty for a slow test.
	 *
	 * @return the smoke test image.
	 */
	const FSlateBrush* GetSmokeTestImage( ) const;

	/**
	 * Returns the tool tip for the automation test result.
	 *
	 * @param ClusterIndex The cluster index.
	 * @return The tooltip text.
	 */
	FText GetTestToolTip( int32 ClusterIndex ) const;

	/**
	* Is the test enabled.
	*
	* @return true if the test is enabled.
	*/
	ECheckBoxState IsTestEnabled() const;

	/**
	* Is the test inside exclude list. This info is taken from Config/DefaultEngine.ini, section [AutomationTestExcludelist].
	*
	* Return the visibility state based on skipped state.
	*/
	EVisibility IsToBeSkipped_GetVisibility() const;

	/**
	* Change opacity of skipped button if it is hovered.
	*
	* Return the Linear Color with different opacity based on hovered status.
	*/
	FSlateColor IsToBeSkipped_GetColorAndOpacity() const;

	/**
	* Is the test inside exclude list through direct exclusion (not through propagation).
	*
	* @return true if the test is inside the exclude list by direct exclusion.
	*/
	bool IsDirectlyExcluded() const;

	/**
	* Return the visibility state based on exclude state.
	*/
	EVisibility IsDirectlyExcluded_GetVisibility() const;

	/**
	* Reason for the exclusion
	*
	* @return the exclusion reason
	*/
	FText GetExcludeReason() const;

	/**
	* Add or remove test from exclude list. It will change Config/DefaultEngine.ini, section [AutomationTestExcludelist].
	*
	* @return respond of dialog
	*/
	FReply SetSkipFlag();

	/**
	* Can the test skip flag be changed.
	*
	* @return true if the test skip flag can be changed.
	*/
	bool CanSkipFlagBeChanged() const;

	/**
	* Open Exclude test options editor
	*
	* @return respond of dialog
	*/
	FReply OnEditExcludeOptionsClicked();

	/**
	 * Returns whether of not test tags can be edited on test.
	*/
	EVisibility CanEditTags_GetVisibility() const;

	/**
	* Open Test Tags editor.
	* 
	* @return Editor reply result.
	*/
	FReply OnEditTestTagsClicked();

	/**
	 * Returns color that indicates test status per cluster.
	 *
	 * @param ClusterIndex The cluster index.
	 * @return The background Color.
	 */
	FSlateColor ItemStatus_BackgroundColor( const int32 ClusterIndex ) const;

	/**
	 * Returns color the duration the test ran for.
	 *
	 * @return The duration the test took as text.
	 */
	FText ItemStatus_DurationText( ) const;

	/**
	 * Helper to ensure throbber is visible when in process and icon is visible otherwise.
	 *
	 * @param ClusterIndex The cluster index.
 	 * @param bForInProcessThrobber If throbbing is in process.
	 * @return The visibility state.
	 */
	EVisibility ItemStatus_GetStatusVisibility( const int32 ClusterIndex, const bool bForInProcessThrobber ) const;

	/**
	 * Helper for internal nodes to ensure throbber is visible when children are in process and icon is visible when children completed.
	 *
	 * @param ClusterIndex The cluster index.
	 * @param bForInProcessThrobber If throbbing is in process.
	 * @return The visibility state.
	 */
	EVisibility ItemStatus_GetChildrenStatusVisibility( const int32 ClusterIndex, const bool bForInProcessThrobber ) const;

	/**
	 * Color of progress bar for internal tree test nodes.
	 *
	 * @param ClusterIndex The cluster index.
	 * @return The progress bar color.
	 */
	FSlateColor ItemStatus_ProgressColor( const int32 ClusterIndex ) const;

	/**
	 * Returns percent completion for an internal tree node for all enabled child tests.
	 *
	 * @param ClusterIndex The cluster index.
	 * @return The percent complete.
	 */
	TOptional<float> ItemStatus_ProgressFraction( const int32 ClusterIndex ) const;

	/**
	 * Returns image that denotes the status of a particular test on the given platform cluster.
	 *
	 * @param ClusterIndex The cluster index.
	 * @return the status image.
	 */
	const FSlateBrush* ItemStatus_StatusImage( const int32 ClusterIndex ) const;

	/**
	 * Returns image that denotes the status of a test category as implied by its children, on the given platform cluster.
	 *
	 * @param ClusterIndex The cluster index.
	 * @return the status image.
	 */
	const FSlateBrush* ItemChildrenStatus_StatusImage(const int32 ClusterIndex) const;
	
	/**
	 * The number of participants required for this test item in string form.
	 *
	 * @return The number of participants needed.
	 */
	FText ItemStatus_NumParticipantsRequiredText( ) const;

private:

	/** Handle the testing checkbox click. */
	void HandleTestingCheckbox_Click( ECheckBoxState );

private:

	/** The column width. */
	float ColumnWidth;

	/** Is Session local. */
	bool IsLocalSession;

	/** Is Skip button hovered. */
	bool bIsToBeSkippedButtonHovered = false;

	/** Holds the highlight string for the automation test. */
	TAttribute<FText> HighlightText;

	/** Holds the automation report. */
	TSharedPtr<IAutomationReport> TestStatus;

	/** Holds a delegate to be invoked when the check box state changed. */
	FOnItemCheckedStateChanged OnCheckedStateChangedDelegate;

	SHorizontalBox::FSlot* TagsSlot;
};
