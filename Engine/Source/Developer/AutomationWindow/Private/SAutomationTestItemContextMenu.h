// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformMisc.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "IAutomationControllerModule.h"

#if WITH_EDITOR

#define LOCTEXT_NAMESPACE "SAutomationTestItemContextMenu"

class SAutomationTestItemContextMenu
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SAutomationTestItemContextMenu) {}
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InSessionManager The session to use.
	 */
	void Construct( const FArguments& InArgs, const TArray<FString>& InAssetNames, const TArray<FString>& InTestNames )
	{
		AssetNames = InAssetNames;
		TestNames = InTestNames;

		ChildSlot
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Content()
				[
					MakeContextMenu( )
				]
		];
	}

protected:

	/**
	 * Builds the context menu widget.
	 *
	 * @return The context menu.
	 */
	TSharedRef<SWidget> MakeContextMenu( )
	{
		FMenuBuilder MenuBuilder(true, NULL);

		MenuBuilder.BeginSection("AutomationOptions", LOCTEXT("MenuHeadingText", "Automation Options"));
		{
			if (TestNames.Num())
			{
				MenuBuilder.AddMenuEntry(LOCTEXT("AutomationMenuEntryCopyTestNameText", "Copy test name(s)"), FText::GetEmpty(), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(this, &SAutomationTestItemContextMenu::HandleContextItemCopyName)));
			}
			if (AssetNames.Num())
			{
				MenuBuilder.AddMenuEntry(LOCTEXT("AutomationMenuEntryLoadText", "Load the asset(s)"), FText::GetEmpty(), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(this, &SAutomationTestItemContextMenu::HandleContextItemTerminate)));
			}
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

private:

	/** Handle the context menu closing down. If an asset is selected, request that it gets loaded */
	void HandleContextItemTerminate( )
	{
		for (int32 AssetIndex = 0; AssetIndex < AssetNames.Num(); ++AssetIndex)
		{
			FModuleManager::GetModuleChecked<IAutomationControllerModule>("AutomationController").GetAutomationController()->RequestLoadAsset(AssetNames[AssetIndex]);
		}
	}

	/** Handle the context menu closing down. Copy the test names to clipboard */
	void HandleContextItemCopyName()
	{
		FPlatformApplicationMisc::ClipboardCopy(*FString::Join(TestNames, TEXT("\n")));
	}

private:

	/** Holds the selected asset name. */
	TArray<FString> AssetNames;

	//** Holds the test names. */
	TArray<FString> TestNames;
};


#undef LOCTEXT_NAMESPACE

#endif //WITH_EDITOR
