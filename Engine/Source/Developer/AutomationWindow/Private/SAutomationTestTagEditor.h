// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "Misc/CoreDelegates.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SSuggestionTextBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "IAutomationControllerModule.h"
#include "SAutomationTestTagListBox.h"
#include "SourceControlHelpers.h"
#include "AutomationWindowStyle.h"

#include "SlateOptMacros.h"

#define LOCTEXT_NAMESPACE "SAutomationTestTagEditor"

DEFINE_LOG_CATEGORY_STATIC(LogAutomationTestTagEditor, Log, All);

namespace TestNameAndTagTableConstants
{
	const FName Name(TEXT("Name"));
	const FName Tags(TEXT("Tags"));
};

class STestNameAndTagTableRow : public SMultiColumnTableRow<TSharedPtr<FString>>
{
public:
	SLATE_BEGIN_ARGS(STestNameAndTagTableRow) {}
		SLATE_ARGUMENT(TSharedPtr<IAutomationReport>, TestReport)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		TestReport = InArgs._TestReport;

		SMultiColumnTableRow<TSharedPtr<FString>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == TestNameAndTagTableConstants::Name)
		{
			if (!TestReport->GetSourceFile().IsEmpty())
			{
				return SNew(SHyperlink)
					.Style(FAutomationWindowStyle::Get(), "Common.GotoNativeCodeHyperlink")
					.OnNavigate_Lambda([this]
						{
							FSlateApplication::Get()
								.GotoLineInSource(TestReport->GetSourceFile(), TestReport->GetSourceFileLine());
						})
					.Text(FText::FromString(TestReport->GetFullTestPath()));
			}
			return SNew(STextBlock)
				.Text(FText::FromString(TestReport->GetFullTestPath()))
				.ToolTipText(FText::FromString(TestReport->GetFullTestPath()));
		}
		else if (ColumnName == TestNameAndTagTableConstants::Tags)
		{
			return SNew(STextBlock)
				.Text(FText::FromString(TestReport->GetTags()))
				.Margin(FMargin(10, 0, 0, 0))
				.ToolTipText(FText::FromString(TestReport->GetTags()));
		}
		return SNullWidget::NullWidget;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:

	IAutomationReportPtr TestReport;
};

DECLARE_DELEGATE(FOnTestTagsSavedEvent);

class SAutomationTestTagEditor
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SAutomationTestTagEditor) {}
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	void Construct(const FArguments& InArgs, const TArray<IAutomationReportPtr>& InTestReports)
	{
		TestReports = InTestReports;

		TArray<FString> ExistingTags;
		TSet<FString> ImmutableTags;
		if (InTestReports.Num() == 1)
		{
			IAutomationReportPtr Report = InTestReports[0];
			ExistingTags = Report->GetTagsArray();

			for (FString TagEx : ExistingTags)
			{
				if (FAutomationTestFramework::Get().IsTagImmutable(Report->GetFullTestPath(), TagEx))
				{
					ImmutableTags.Add(TagEx);
				}
			}
		}

		ChildSlot
			[
				SNew(SBox)
					.WidthOverride(640)
					[
						SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.MaxHeight(200.f)
							.Padding(5.0f)
							[
								SNew(SListView<TSharedPtr<IAutomationReport>>)
									.SelectionMode(ESelectionMode::None)
									.ListItemsSource(&InTestReports)
									.OnGenerateRow(this, &SAutomationTestTagEditor::OnGenerateTestNameAndTagRow)
									.HeaderRow
									(
										SNew(SHeaderRow)
										+ SHeaderRow::Column(TestNameAndTagTableConstants::Name)
										.FillWidth(0.7f)
										.HAlignHeader(HAlign_Center)
										.VAlignHeader(VAlign_Center)
										.HAlignCell(HAlign_Left)
										.VAlignCell(VAlign_Center)
										[
											SNew(SHorizontalBox)
												+ SHorizontalBox::Slot()
												.AutoWidth()
												.VAlign(VAlign_Center)
												[
													SNew(STextBlock)
														.Text(LOCTEXT("TestName_Header", "Test"))
												]
										]

										+ SHeaderRow::Column(TestNameAndTagTableConstants::Tags)
										.FillWidth(0.3f)
										.HAlignHeader(HAlign_Center)
										.VAlignHeader(VAlign_Center)
										.HAlignCell(HAlign_Left)
										.VAlignCell(VAlign_Center)
										[
											SNew(SHorizontalBox)
												+ SHorizontalBox::Slot()
												.AutoWidth()
												.VAlign(VAlign_Center)
												[
													SNew(STextBlock)
														.Text(LOCTEXT("TestTags_Header", "Tags"))
												]
										]
									)
							]
							+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(5.0f)
							[
								SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.FillWidth(0.9f)
									.VAlign(VAlign_Center)
									[
										SAssignNew(TagSuggestionBox, SSuggestionTextBox)
											.ForegroundColor(FSlateColor::UseForeground())
											.CommitOnSuggestionSelected(true)
											.ClearKeyboardFocusOnCommit(false)
											.OnTextChanged(this, &SAutomationTestTagEditor::TagSuggestionBox_TextChanged)
											.OnTextCommitted(this, &SAutomationTestTagEditor::TagSuggestionBox_SetText)
											.HintText(LOCTEXT("EnterTagHint", "Enter tag"))
											.OnShowingSuggestions(this, &SAutomationTestTagEditor::TagSuggestionBox_GetSuggestions)
									]
									+ SHorizontalBox::Slot()
									.FillWidth(0.1f)
									.Padding(FMargin(1, 0, 0, 0))
									.VAlign(VAlign_Center)
									[
										SAssignNew(AddButton, SButton)
											.Text(LOCTEXT("AddTags", "Add"))
											.IsEnabled(false)
											.OnClicked(this, &SAutomationTestTagEditor::OnAddButtonClicked)
									]
							]
							+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(5.0f)
							[
								SAssignNew(TagListBox, SAutomationTestTagListBox, ExistingTags, ImmutableTags)
							]
							+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(5.0f)
							.HAlign(HAlign_Center)
							[
								SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.AutoWidth()
									.HAlign(HAlign_Center)
									.VAlign(VAlign_Center)
									.Padding(5, 0)
									[
										SNew(SButton)
											.Text(LOCTEXT("SaveTags", "Save"))
											.IsEnabled(this, &SAutomationTestTagEditor::SaveButonEnabled)
											.OnClicked(this, &SAutomationTestTagEditor::OnSaveButtonClicked)
									]
									+ SHorizontalBox::Slot()
									.AutoWidth()
									.HAlign(HAlign_Center)
									.VAlign(VAlign_Center)
									.Padding(5, 0)
									[
										SNew(SButton)
											.Text(LOCTEXT("CancelTags", "Discard"))
											.OnClicked(this, &SAutomationTestTagEditor::OnDiscardButtonClicked)
									]
							]
					]
			];
	}

	static void ShowInWindow(const TSharedRef<const SWidget> InParentWindow, const TArray<TSharedPtr<IAutomationReport>>& InTestReports, const FOnTestTagsSavedEvent& OnTestTagsSaved)
	{
		TSharedPtr<SAutomationTestTagEditor> TestTagEditor;
		TSharedRef<SWidget> TagsForm = SNew(SBox)
			.WidthOverride(640)
			[
				SAssignNew(TestTagEditor, SAutomationTestTagEditor, InTestReports)
			];

		TSharedRef<SWindow> Window = SNew(SWindow)
			.Title(LOCTEXT("TestTagsEditor", "Edit Tags"))
			.SizingRule(ESizingRule::Autosized)
			.ClientSize(FVector2D(480, 160))
			.SupportsMinimize(false)
			.SupportsMaximize(false)
			.AutoCenter(EAutoCenter::PrimaryWorkArea);

		Window->SetContent
		(
			TagsForm
		);

		TestTagEditor->SetOnTestTagsSavedEvent(OnTestTagsSaved);

		FSlateApplication::Get().AddModalWindow(Window, InParentWindow);
	}

	void SetOnTestTagsSavedEvent(const FOnTestTagsSavedEvent& Event)
	{
		OnTestTagsSavedEvent = Event;
	}

private:
	void AddTagToListBox(const FString& InTag)
	{
		TagListBox->AddTag(InTag, true, false);
		TagSuggestionBox->SetText(FText::GetEmpty());
	}

	FReply OnAddButtonClicked()
	{
		FString SelectedTag = TagSuggestionBox->GetText().ToString();
		if (!SelectedTag.IsEmpty())
		{
			AddTagToListBox(SelectedTag);
		}
		return FReply::Handled();
	}

	void TagSuggestionBox_SetText(const FText& NewText, ETextCommit::Type CommitInfo)
	{
		if (CommitInfo == ETextCommit::Type::OnEnter && !NewText.ToString().IsEmpty())
		{
			AddTagToListBox(NewText.ToString());
		}
	}

	void TagSuggestionBox_TextChanged(const FText& NewText)
	{
		AddButton->SetEnabled(!NewText.IsEmpty());
	}

	void TagSuggestionBox_GetSuggestions(const FString& CurrText, TArray<FString>& OutSuggestions)
	{
		OutSuggestions = FAutomationTestFramework::Get().GetAllExistingTags().Array().FilterByPredicate([&CurrText](const FString& CurrentTag) { return CurrentTag.Contains(CurrText); });
	}

	FReply OnDiscardButtonClicked()
	{
		TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(this->AsShared());

		TArray<FString> NewTags = TagListBox->GetNewTags();
		TArray<FString> MarkedForDeleteTags = TagListBox->GetMarkForDeleteTags();
		if (NewTags.Num() > 0 || MarkedForDeleteTags.Num() > 0)
		{
			if (EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, FText::FromString(TEXT("Discard changes?"))))
			{
				FSlateApplication::Get().DestroyWindowImmediately(Window.ToSharedRef());
			}
		}
		else
		{
			FSlateApplication::Get().DestroyWindowImmediately(Window.ToSharedRef());
		}
		return FReply::Handled();
	}

	bool SaveButonEnabled() const
	{
		return TagListBox->GetNewTags().Num() > 0 || TagListBox->GetMarkForDeleteTags().Num() > 0;
	}


	FReply OnSaveButtonClicked()
	{
		TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(this->AsShared());

		TArray<FString> NewTags = TagListBox->GetNewTags();
		TArray<FString> MarkedForDeleteTags = TagListBox->GetMarkForDeleteTags();
		
		if (NewTags.Num() > 0 || MarkedForDeleteTags.Num() > 0)
		{
			FString Changes;
			if (NewTags.Num() > 0)
			{
				Changes = Changes.Append(TEXT("New tags: ")).Append(FString::Join(NewTags, TEXT(","))).Append(TEXT(".\n"));
			}
			if (MarkedForDeleteTags.Num() > 0)
			{
				Changes = Changes.Append(TEXT("Removing tags: ")).Append(FString::Join(MarkedForDeleteTags, TEXT(","))).Append(TEXT("."));
			}
			FText ChangesText = FText::FromString(Changes);
			if (EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, FText::Format(LOCTEXT("TagsChangedSaveMsg", "Confirm tag changes to apply.\n{0}"), ChangesText)))
			{
				TArray<FString> Tags;
				TArray<FString> TestNames;
				TArray<FString> TestFilePaths;
				for (IAutomationReportPtr Report : TestReports)
				{
					if (!Report->GetAssetPath().IsEmpty())
					{
						FString PackageLocalPath;
						if (FPackageName::TryConvertLongPackageNameToFilename(*Report->GetAssetPath(), PackageLocalPath, FPackageName::GetAssetPackageExtension()))
						{
							TestFilePaths.Add(PackageLocalPath);
						}
						else
						{
							UE_LOGF(LogAutomationTestTagEditor, Error, "Could not resolve asset path for test %ls, skipping from tag mapping", *Report->GetFullTestPath());
							continue;
						}
					}
					else
					{
						TestFilePaths.Add(Report->GetSourceFile());
					}
					TestNames.Add(Report->GetFullTestPath());

					Tags = Report->GetTagsArray();
					for (FString DeleteTag : MarkedForDeleteTags)
					{
						NewTags.Remove(DeleteTag);
						if (FAutomationTestFramework::Get().IsTagImmutable(Report->GetFullTestPath(), DeleteTag) && Tags.Contains(DeleteTag))
						{
							// Do not delete own immutable tag
							continue;
						}
						Tags.Remove(DeleteTag);
					}
					Tags.Append(NewTags);
					bool Registered = FAutomationTestFramework::Get().RegisterAutomationTestTags(
						Report->GetFullTestPath(),
						FString::JoinBy(Tags, TEXT(""),
							[](const FString& InTag)
							{
								return FString::Printf(TEXT("[%s]"), *InTag);
							}), false);
					check(Registered);
				}

				FAutomationTestFramework::Get().SaveTestTagMappings(TestNames, TestFilePaths, FBeforeTagMappingConfigSaved::CreateLambda(
						[](const FString& ConfigPath)
						{
							if (SourceControlHelpers::IsAvailable())
							{
								if (IFileManager::Get().FileExists(*ConfigPath))
								{
									SourceControlHelpers::SyncFile(ConfigPath);
								}
							}
						}),
						FAfterTagMappingConfigSaved::CreateLambda([](const FString& ConfigPath)
						{
							if (SourceControlHelpers::IsAvailable())
							{
								SourceControlHelpers::CheckOutOrAddFile(ConfigPath);
							}
						}));

				OnTestTagsSavedEvent.ExecuteIfBound();

				FSlateApplication::Get().DestroyWindowImmediately(Window.ToSharedRef());
			}
		}
		return FReply::Handled();
	}

	TSharedRef<ITableRow> OnGenerateTestNameAndTagRow(TSharedPtr<IAutomationReport> InTestReport, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(STestNameAndTagTableRow, OwnerTable).TestReport(InTestReport);
	}

	void OnGetTestNameAndTagChildren(TSharedPtr<IAutomationReport> InItem, TArray<TSharedPtr<IAutomationReport>>& OutItems)
	{
		OutItems = InItem.Get()->GetChildReports();
	}

	TSharedPtr<SSuggestionTextBox> TagSuggestionBox;
	TSharedPtr<SAutomationTestTagListBox> TagListBox;
	TSharedPtr<SButton> AddButton;
	TArray<IAutomationReportPtr> TestReports;
	FOnTestTagsSavedEvent OnTestTagsSavedEvent;
};


#undef LOCTEXT_NAMESPACE