// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FilterCollection.h"
#include "Misc/MessageDialog.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SWindow.h"
#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "IAutomationControllerModule.h"

#include "IAutomationReport.h"

#include "AutomationGroupFilter.h"
#include "AutomationWindowStyle.h"

#define LOCTEXT_NAMESPACE "SAutomationTestTagFilter"

DECLARE_DELEGATE(FOnTestFilterAppliedEvent);

class SAutomationTestTagFilter
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SAutomationTestTagFilter) {}
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<FAutomationGroupFilter> InAutomationTagFilter)
	{
		AutomationTagFilter = InAutomationTagFilter;
		ChildSlot
			[
				SNew(SBox)
					.WidthOverride(300)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.MinWidth(150.f)
						.MaxWidth(250.f)
						.AutoWidth()
						.Padding(5.0f)
						[
							SAssignNew(FilterText, SEditableTextBox)
								.ForegroundColor(FSlateColor::UseForeground())
								.ClearKeyboardFocusOnCommit(false)
								.HintText(LOCTEXT("EnterFilterHint", "Enter tag filter"))
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.Padding(5, 0)
						[
							SNew(SButton)
								.ToolTipText(LOCTEXT("ApplyTagFilterHint", "Apply Filter"))
								.IsEnabled_Lambda([this]()
									{
										return !FilterText->GetText().IsEmptyOrWhitespace();
									})
								.OnClicked(this, &SAutomationTestTagFilter::OnApplyTagFilterButtonClicked)
								[
									SAssignNew(FilterButtonImage, SLayeredImage)
										.Image(FAutomationWindowStyle::Get().GetBrush("AutomationWindow.ApplyTagFilter"))
								]
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.Padding(5, 0)
						[
							SNew(SButton)
								.ToolTipText(LOCTEXT("RemoveTagFilterHint", "Remove Filter"))
								.IsEnabled_Lambda([this]()
									{
										return IsFilterApplied;
									})
								.OnClicked(this, &SAutomationTestTagFilter::OnRemoveTagFilterButtonClicked)
								[
									SNew(SImage)
										.Image(FAutomationWindowStyle::Get().GetBrush("AutomationWindow.RemoveTagFilter"))
								]
						]
					]
			];

		FilterButtonImage->AddLayer(TAttribute<const FSlateBrush*>::CreateLambda([this]() -> const FSlateBrush*
			{
				if (!IsFilterApplied)
				{
					return nullptr;
				}

				return FAutomationWindowStyle::Get().GetBrush("AutomationWindow.TagFilterIsActive");
			}));
	}

	FReply OnApplyTagFilterButtonClicked()
	{
		FString FilterTextString = FilterText->GetText().ToString();
		if (!FilterTextString.IsEmpty())
		{
			ApplyTagFilter(FilterText->GetText().ToString());
			OnTestFilterApplied.ExecuteIfBound();
			IsFilterApplied = true;
		}
		return FReply::Handled();
	}

	FReply OnRemoveTagFilterButtonClicked()
	{
		ClearFilter();
		return FReply::Handled();
	}

	void ApplyTagFilter(const FString& TagFilter)
	{
		TArray<FAutomatedTestTagFilter> TagList;
		TagList.Add(FAutomatedTestTagFilter(TagFilter));
		AutomationTagFilter->SetTagFilter(TagList);
	}

	void SetOnTestFilterAppliedEvent(const FOnTestFilterAppliedEvent& OnTestFilterAppliedEvent)
	{
		OnTestFilterApplied = OnTestFilterAppliedEvent;
	}

	void ClearFilter()
	{
		ApplyTagFilter(TEXT(""));
		OnTestFilterApplied.ExecuteIfBound();
		FilterText->SetText(FText::GetEmpty());
		IsFilterApplied = false;
	}

protected:
	TSharedPtr<FAutomationGroupFilter> AutomationTagFilter;
	FOnTestFilterAppliedEvent OnTestFilterApplied;
	TSharedPtr<SEditableTextBox> FilterText;
	TSharedPtr<SLayeredImage> FilterButtonImage;
	bool IsFilterApplied;
};


#undef LOCTEXT_NAMESPACE