// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "IAutomationControllerModule.h"

#define LOCTEXT_NAMESPACE "SAutomationTestTagListBox"

DEFINE_LOG_CATEGORY_STATIC(LogAutomationTestTagListBox, Log, All);

class SAutomationTestTagItem
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SAutomationTestTagItem) {}
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	void Construct(const FArguments& InArgs, const FString& InTag, const bool InIsNew, const bool InImmutable)
	{
		TagValue = InTag;
		IsNew = InIsNew;
		FText TagText = FText::Format(LOCTEXT("NewTestTag", "{0}"), FText::FromString(InTag));
		FText TagToolTipText = FText::Format(LOCTEXT("NewTestTagToolTip", "{0}"),
			FText::FromString(InTag + (InImmutable ? TEXT(" (immutable, from source code)") : TEXT(""))));

		ChildSlot
			[
				SAssignNew(MainBorder, SBorder)
					.IsEnabled(!InImmutable)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.BorderBackgroundColor(FColorList::Black)
					.Content()
					[
						SNew(SHorizontalBox)
							.Clipping(EWidgetClipping::ClipToBoundsWithoutIntersecting)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(EVerticalAlignment::VAlign_Center)
							.HAlign(EHorizontalAlignment::HAlign_Left)
							.Padding(4, 2, 2, 2)
							[
								SNew(STextBlock)
									.ColorAndOpacity(FLinearColor(.8f, .8f, .8f, 1.0f))
									.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
									.Text(TagText)
									.ToolTipText(TagToolTipText)
							]
							+ SHorizontalBox::Slot()
							.Padding(2)
							[
								SNew(SButton)
									.Visibility(!InImmutable ? EVisibility::Visible : EVisibility::Collapsed)
									.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
									.OnClicked(this, &SAutomationTestTagItem::ToggleTagDeletion)
									.VAlign(VAlign_Center)
									.HAlign(HAlign_Center)
									[
										SNew(SImage)
											.Image(FAppStyle::GetBrush("Icons.Delete"))
											.ColorAndOpacity(FSlateColor::UseForeground())
									]
							]
					]
			];
	}

	bool IsMarkedForDeletion()
	{
		return MarkedForDeletion;
	}

	FString GetTagValue()
	{
		return TagValue;
	}

protected:
	FReply ToggleTagDeletion();

	TSharedPtr<SBorder> MainBorder;
	bool IsNew;
	bool MarkedForDeletion;
	FString TagValue;
};

class SAutomationTestTagListBox
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SAutomationTestTagListBox) {}
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InCurrentTestStatus The current test report pointer.
	 */
	void Construct( const FArguments& InArgs, TArray<FString>& InExistingTags, TSet<FString> InImmutableTags )
	{
		ExistingTags = InExistingTags;

		ChildSlot
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.BorderBackgroundColor(FLinearColor::Transparent)
				.Content()
				[
					SAssignNew(TagBox, SWrapBox)
						.UseAllottedSize(true)
						.Clipping(EWidgetClipping::ClipToBoundsWithoutIntersecting)
				]
		];

		for (const FString& TestTag : ExistingTags)
		{
			AddTag(TestTag, false, InImmutableTags.Contains(TestTag));
		}
	}

	void AddTag(const FString& TestTag, bool IsNew, bool IsImmutable)
	{
		bool AlreadyInNewTagsCaseInsensitive =
			NewTags.FindByPredicate([TestTag](const FString& Item)
				{
					return Item.Compare(TestTag, ESearchCase::IgnoreCase) == 0;
				}) != nullptr;
		bool AlreadyInExistingTagsCaseInsensitive =
			ExistingTags.FindByPredicate([TestTag](const FString& Item)
				{
					return Item.Compare(TestTag, ESearchCase::IgnoreCase) == 0;
				}) != nullptr;

		if (AlreadyInNewTagsCaseInsensitive || (IsNew && AlreadyInExistingTagsCaseInsensitive))
			return;

		if (IsNew)
		{
			NewTags.Add(TestTag);
		}
		TagBox->AddSlot()
			.FillEmptySpace(false)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.Padding(FMargin(2.0f))
			[
				SNew(SAutomationTestTagItem, TestTag, IsNew, IsImmutable)
			];
	}

	TArray<FString> GetNewTags()
	{
		return NewTags;
	}

	int32 RemoveNewTagsItem(const FString &ItemToRemove)
	{
		return NewTags.RemoveSingle(ItemToRemove);
	}

	TArray<FString> GetMarkForDeleteTags()
	{
		TArray<FString> ToDelete;
		TagBox->GetAllChildren()->ForEachWidget([this, &ToDelete](SWidget& TagItem)
			{
				SAutomationTestTagItem* TestTagItem = (SAutomationTestTagItem*)(&TagItem);
				if (TestTagItem && TestTagItem->IsMarkedForDeletion())
				{
					ToDelete.Add(TestTagItem->GetTagValue());
				}
			});

		return ToDelete;
	}

private:

	TArray<FString> ExistingTags;
	TArray<FString> NewTags;
	TSharedPtr<SWrapBox> TagBox;
};


inline FReply SAutomationTestTagItem::ToggleTagDeletion()
{
	TSharedPtr<SWidget> TagListBoxWidget = MainBorder;

	do
	{
		TagListBoxWidget = TagListBoxWidget->GetParentWidget();

		if (TagListBoxWidget == nullptr)
		{
			UE_LOGF(LogAutomationTestTagListBox, Error, "Could not find parent widget with type SAutomationTestTagListBox");

			return FReply::Handled();
		}

	} while (!TagListBoxWidget->GetTypeAsString().Equals("SAutomationTestTagListBox"));

	const TSharedPtr<SAutomationTestTagListBox> TagListBox = StaticCastSharedPtr<SAutomationTestTagListBox>(TagListBoxWidget);
	
	if (TagListBox->RemoveNewTagsItem(TagValue) == 0)
	{
		MarkedForDeletion = !MarkedForDeletion;
	}
	
	if (!IsNew)
	{
		MainBorder->SetBorderBackgroundColor(MarkedForDeletion ? FColorList::Red : FColorList::Black);
	}
	else
	{
		ChildSlot.DetachWidget();
	}
	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE