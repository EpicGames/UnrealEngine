// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAutomationWindow.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformApplicationMisc.h"
#include "PlatformInfo.h"
#include "Misc/MessageDialog.h"
#include "Misc/TextFilter.h"
#include "Misc/FilterCollection.h"
#include "Widgets/Layout/SSplitter.h"
#include "SlateOptMacros.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "SAutomationWindowCommandBar.h"
#include "AutomationFilter.h"
#include "AutomationGroupFilter.h"
#include "AutomationPresetManager.h"
#include "SAutomationTestItemContextMenu.h"
#include "SAutomationTestItem.h"
#include "SSimpleComboButton.h"

#if WITH_EDITOR
	#include "Dialogs/Dialogs.h"
	#include "Engine/Level.h"
	#include "Engine/World.h"
	#include "FileHelpers.h"
	#include "AssetRegistry/AssetRegistryModule.h"
#endif

#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SHyperlink.h"
#include "Internationalization/Regex.h"
#include "AutomationWindowStyle.h"
#include "AutomationControllerSettings.h"
#include "Framework/Notifications/NotificationManager.h"

#include "SAutomationTestTagEditor.h"
#include "SAutomationTestTagFilter.h"


#define LOCTEXT_NAMESPACE "AutomationTest"


//////////////////////////////////////////////////////////////////////////
// FAutomationWindowCommands

class FAutomationWindowCommands : public TCommands<FAutomationWindowCommands>
{
public:
	FAutomationWindowCommands()
		: TCommands<FAutomationWindowCommands>(
		TEXT("AutomationWindow"),
		NSLOCTEXT("Contexts", "AutomationWindow", "Automation Window"),
		NAME_None, FAutomationWindowStyle::Get().GetStyleSetName()
		)
	{
	}

	virtual void RegisterCommands() override
	{
		UI_COMMAND( RefreshTests, "Refresh Tests", "Refresh Tests", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( FindWorkers, "Find Workers", "Find Workers", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( ErrorFilter, "Errors", "Toggle Error Filter", EUserInterfaceActionType::ToggleButton, FInputChord() );
		UI_COMMAND( WarningFilter, "Warnings", "Toggle Warning Filter", EUserInterfaceActionType::ToggleButton, FInputChord() );
		UI_COMMAND( DeveloperDirectoryContent, "Dev Content", "Developer Directory Content Filter (when enabled, developer directories are also included)", EUserInterfaceActionType::ToggleButton, FInputChord() );
		UI_COMMAND( ExcludedTestsFilter, "Excluded Tests", "Toggle Excluded Tests only", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND( EditTags, "Edit Tags for Selection", "Edit Tags for Selection", EUserInterfaceActionType::Button, FInputChord());

#if WITH_EDITOR
		// Added button for running the currently open level test.
		UI_COMMAND(RunLevelTest, "Run Level Test", "Run Level Test", EUserInterfaceActionType::Button, FInputChord());
#endif

		UI_COMMAND(CreateNewPreset, "Create Automation Preset", "Create New Preset", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(RemoveCurrentPreset, "Delete", "Remove Current Preset", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(RenameCurrentPreset, "Rename", "Rename Current Preset", EUserInterfaceActionType::Button, FInputChord());
	}
public:
	TSharedPtr<FUICommandInfo> RefreshTests;
	TSharedPtr<FUICommandInfo> FindWorkers;
	TSharedPtr<FUICommandInfo> ErrorFilter;
	TSharedPtr<FUICommandInfo> WarningFilter;
	TSharedPtr<FUICommandInfo> DeveloperDirectoryContent;
	TSharedPtr<FUICommandInfo> ExcludedTestsFilter;
	TSharedPtr<FUICommandInfo> EditTags;

#if WITH_EDITOR
	TSharedPtr<FUICommandInfo> RunLevelTest;
#endif

	TSharedPtr<FUICommandInfo> CreateNewPreset;
	TSharedPtr<FUICommandInfo> RemoveCurrentPreset;
	TSharedPtr<FUICommandInfo> RenameCurrentPreset;
};

//////////////////////////////////////////////////////////////////////////
// SAutomationWindow

SAutomationWindow::SAutomationWindow() 
	: ColumnWidth(90.0f)
	, bIsLabelVisibilityEnabled(false)
	, bAutoExpandSingleItemSubgroups(true)
{
	UAutomationControllerSettings* Settings = UAutomationControllerSettings::StaticClass()->GetDefaultObject<UAutomationControllerSettings>();
	if (nullptr == Settings)
	{
		return;
	}

	bAutoExpandSingleItemSubgroups = Settings->bAutoExpandSingleItemSubgroups;
}

SAutomationWindow::~SAutomationWindow()
{
	// @todo PeterMcW: is there an actual delegate missing here?
	//give the controller a way to indicate it requires a UI update
	//AutomationController->SetRefreshTestCallback(FOnAutomationControllerTestsRefreshed());

	// Remove ourselves from the session manager
	if( SessionManager.IsValid( ) )
	{
		SessionManager->OnCanSelectSession().RemoveAll(this);
		SessionManager->OnSelectedSessionChanged().RemoveAll(this);
		SessionManager->OnSessionInstanceUpdated().RemoveAll(this);
	}

	if (AutomationController.IsValid())
	{
		AutomationController->RemoveCallbacks();

		AutomationController->OnControllerReset().RemoveAll(this);
		AutomationController->OnTestsRefreshed().RemoveAll(this);
		AutomationController->OnTestsAvailable().RemoveAll(this);
		AutomationController->OnTestsComplete().RemoveAll(this);
	}

#if WITH_EDITOR
	if ( FModuleManager::Get().IsModuleLoaded(TEXT("AssetRegistry")) )
	{
		IAssetRegistry* AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnFileLoadProgressUpdated().RemoveAll(this);
		}
	}
#endif

	TSharedPtr<IAutomationReport> PreviousSelectionLock = PreviousSelection.Pin();
	if ( PreviousSelectionLock.IsValid() )
	{
		PreviousSelectionLock->OnSetResults.Unbind();
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SAutomationWindow::Construct( const FArguments& InArgs, const IAutomationControllerManagerRef& InAutomationController, const TSharedRef<ISessionManager>& InSessionManager )
{
	FAutomationWindowCommands::Register();
	CreateCommands();

#if WITH_EDITOR
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnFilesLoaded().AddSP(this, &SAutomationWindow::OnAssetRegistryFilesLoaded);
#endif

	TestPresetManager = MakeShareable(new FAutomationTestPresetManager());
	TestPresetManager->LoadPresets();
	bAddingTestPreset = false;
	bRenamingTestPreset = false;

	bHasChildTestSelected = false;

	SessionManager = InSessionManager;
	AutomationController = InAutomationController;

	AutomationController->OnControllerReset().AddSP(this, &SAutomationWindow::OnRefreshTestCallback);
	AutomationController->OnTestsRefreshed().AddSP(this, &SAutomationWindow::OnRefreshTestCallback);
	AutomationController->OnTestsAvailable().AddSP(this, &SAutomationWindow::OnTestAvailableCallback);
	AutomationController->OnTestsComplete().AddSP(this, &SAutomationWindow::OnTestsCompleteCallback);

	AutomationControllerState = AutomationController->GetTestState();
	
	//cache off reference to filtered reports
	TArray <TSharedPtr <IAutomationReport> >& TestReports = AutomationController->GetFilteredReports();

	// Create the search filter and set criteria
	AutomationTextFilter = MakeShareable( new AutomationReportTextFilter( AutomationReportTextFilter::FItemToStringArray::CreateSP( this, &SAutomationWindow::PopulateReportSearchStrings ) ) );
	AutomationGeneralFilter = MakeShareable( new FAutomationFilter() ); 
	AutomationGroupFilter = MakeShareable( new FAutomationGroupFilter() );
	AutomationTagFilter = MakeShareable(new FAutomationGroupFilter());
	AutomationFilters = MakeShareable( new AutomationFilterCollection() );
	AutomationFilters->Add( AutomationTextFilter );
	AutomationFilters->Add( AutomationGeneralFilter );
	AutomationFilters->Add( AutomationGroupFilter );
	AutomationFilters->Add( AutomationTagFilter );

	bIsRequestingTests = false;

	//make the widget for platforms
	PlatformsHBox = SNew(SHorizontalBox);

	TestTable = SNew(STreeView< TSharedPtr< IAutomationReport > >)
		.SelectionMode(ESelectionMode::Multi)
		.TreeItemsSource( &TestReports )
		// Generates the actual widget for a tree item
		.OnGenerateRow( this, &SAutomationWindow::OnGenerateWidgetForTest )
		// Gets children
		.OnGetChildren(this, &SAutomationWindow::OnGetChildren)
		// on recursive expansion (shift + click)
		.OnSetExpansionRecursive(this, &SAutomationWindow::OnTestExpansionRecursive)
		//on selection
		.OnSelectionChanged(this, &SAutomationWindow::OnTestSelectionChanged)
		.OnExpansionChanged( this, &SAutomationWindow::OnExpansionChanged)
#if WITH_EDITOR
		// If in editor - add a context menu for opening assets when in editor
		.OnContextMenuOpening(this, &SAutomationWindow::HandleAutomationListContextMenuOpening)
#endif
		.HeaderRow
		(
			SNew(SHeaderRow)
			+ SHeaderRow::Column( AutomationTestWindowConstants::Checked )
			.FixedWidth(30.0f)
			.HAlignHeader(HAlign_Center)
			.VAlignHeader(VAlign_Center)
			.HAlignCell(HAlign_Center)
			.VAlignCell(VAlign_Center)
			[
				//global enable/disable check box
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &SAutomationWindow::HeaderCheckboxStateChange)
				.ToolTipText(LOCTEXT("Enable Disable Test", "Enable / Disable Test"))
			]

			+ SHeaderRow::Column( AutomationTestWindowConstants::Skipped )
			.FixedWidth(30.0f)
			.HAlignHeader(HAlign_Center)
			.VAlignHeader(VAlign_Center)
			.HAlignCell(HAlign_Center)
			.VAlignCell(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.4f))
				.ToolTipText(LOCTEXT("Excluded", "Excluded"))
				.Image(FAutomationWindowStyle::Get().GetBrush("AutomationWindow.ExcludedTestsFilter"))
			]

			+ SHeaderRow::Column( AutomationTestWindowConstants::SkippedOptions )
			.FixedWidth(30.0f)
			.HAlignHeader(HAlign_Center)
			.VAlignHeader(VAlign_Center)
			.HAlignCell(HAlign_Center)
			.VAlignCell(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.4f))
				.Image(FAutomationWindowStyle::Get().GetBrush("Icons.Edit"))
			]

			+ SHeaderRow::Column( AutomationTestWindowConstants::Title )
			.FillWidth(0.80f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew( STextBlock )
					.Text( LOCTEXT("TestName_Header", "Test") )
				]
			]

			+ SHeaderRow::Column(AutomationTestWindowConstants::Tags)
			.FillWidth(0.20f)
			.HAlignHeader(HAlign_Left)
			.VAlignHeader(VAlign_Center)
			.HAlignCell(HAlign_Left)
			.VAlignCell(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(LOCTEXT("TagsHeader", "Tags"))
			]

			+ SHeaderRow::Column( AutomationTestWindowConstants::SmokeTest )
			.FixedWidth( 30.0f )
			.HAlignHeader(HAlign_Center)
			.VAlignHeader(VAlign_Center)
			.HAlignCell(HAlign_Center)
			.VAlignCell(VAlign_Center)
			[
				//icon for the smoke test column
				SNew(SImage)
				.ColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.4f))
				.ToolTipText( LOCTEXT( "Smoke Test", "Smoke Test" ) )
				.Image(FAutomationWindowStyle::Get().GetBrush("Automation.SmokeTest"))
			]
	
			+ SHeaderRow::Column( AutomationTestWindowConstants::RequiredDeviceCount )
			.FixedWidth(30.0f)
			.HAlignHeader(HAlign_Center)
			.VAlignHeader(VAlign_Center)
			.HAlignCell(HAlign_Center)
			.VAlignCell(VAlign_Center)
			[
				SNew( SImage )
				.Image(FAutomationWindowStyle::Get().GetBrush("Automation.ParticipantsWarning") )
				.ToolTipText( LOCTEXT( "RequiredDeviceCountWarningToolTip", "Number of devices required." ) )
			]
	
			+ SHeaderRow::Column(AutomationTestWindowConstants::Timing)
			.FixedWidth( 80.0f )
			.DefaultLabel(LOCTEXT("TestDurationRange", "Duration"))

			+ SHeaderRow::Column( AutomationTestWindowConstants::Status )
			.FixedWidth(ColumnWidth)
			.HAlignHeader(HAlign_Left)
			.VAlignHeader(VAlign_Center)
			[
				//platform header placeholder
				PlatformsHBox.ToSharedRef()
			]
		);

	RequestedFilterComboList.Empty();
	RequestedFilterComboList.Add(MakeShareable(new FString(TEXT("All Tests"))));
	RequestedFilterComboList.Add(MakeShareable(new FString(TEXT("Smoke Tests"))));
	RequestedFilterComboList.Add(MakeShareable(new FString(TEXT("Engine Tests"))));
	RequestedFilterComboList.Add(MakeShareable(new FString(TEXT("Product Tests"))));
	RequestedFilterComboList.Add(MakeShareable(new FString(TEXT("Performance Tests"))));
	RequestedFilterComboList.Add(MakeShareable(new FString(TEXT("Stress Tests"))));
	RequestedFilterComboList.Add(MakeShareable(new FString(TEXT("Standard Tests"))));
	RequestedFilterComboList.Add(MakeShareable(new FString(TEXT("Negative Tests"))));

	UAutomationControllerSettings* Settings = UAutomationControllerSettings::StaticClass()->GetDefaultObject<UAutomationControllerSettings>();
	TArray<FAutomatedTestGroup> TestGroups = Settings->Groups;
	GroupComboList.Empty();
	GroupFiltersMap.Empty();
	const FString AllGroups = TEXT("All Groups");
	GroupComboList.Add(MakeShareable(new FString(AllGroups)));
	GroupFiltersMap.Add(AllGroups, TArray<FAutomatedTestFilter>());
	for (int TestGroupIdx = 0; TestGroupIdx < TestGroups.Num(); TestGroupIdx++)
	{
		GroupComboList.Add(MakeShareable(new FString(TestGroups[TestGroupIdx].Name)));
		GroupFiltersMap.Add(TestGroups[TestGroupIdx].Name, TestGroups[TestGroupIdx].Filters);
	}	
	
	TSharedRef<SNotificationList> NotificationList = SNew(SNotificationList) .Visibility( EVisibility::HitTestInvisible );

	//build the actual guts of the window
	this->ChildSlot
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
			[
				SNew( SSplitter )
				.IsEnabled(this, &SAutomationWindow::HandleMainContentIsEnabled)
				.Orientation(Orient_Vertical)

				+ SSplitter::Slot()
				.Value(0.66f)
				[
					//automation test panel
					SAssignNew( MenuBar, SVerticalBox )

					//ACTIONS
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( SHorizontalBox )

						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						.FillWidth(1)
						[
							SAutomationWindow::MakeAutomationWindowToolBar( AutomationWindowActions.ToSharedRef(), SharedThis(this) )
						]
					]

					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					.Padding(0.0f)
					[
						SNew(SOverlay)
						+ SOverlay::Slot()
						[
							SNew(SBorder)
							.BorderImage(FAutomationWindowStyle::Get().GetBrush("NoBorder"))
							.Padding(3)
							[
								SNew(SBox)
								.Padding(4)
								[
									SNew(SVerticalBox)

									+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(0.0f, 0.0f, 0.0f, 4.0f)
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
										.AutoWidth()
										.VAlign(VAlign_Center)
										[
											SNew(SBox)
											.MinDesiredWidth(130.0f)
											[
												SAssignNew(RequestedFilterComboBox, SComboBox< TSharedPtr<FString> >)
												.OptionsSource(&RequestedFilterComboList)
												.InitiallySelectedItem(RequestedFilterComboList[6])
												.OnGenerateWidget(this, &SAutomationWindow::GenerateRequestedFilterComboItem)
												.OnSelectionChanged(this, &SAutomationWindow::HandleRequesteFilterChanged)
												.ContentPadding(FMargin(4.0, 1.0f))
												[
													SNew(STextBlock)
													.Text(this, &SAutomationWindow::GetRequestedFilterComboText)
												]
											]
										]


										+ SHorizontalBox::Slot()
										.FillWidth(0.02f)
										[
											SNew(SSpacer)
											.Visibility(this, &SAutomationWindow::HandleGroupsVisibility)
										]

										+ SHorizontalBox::Slot()
										.AutoWidth()
										.VAlign(VAlign_Center)
										[
											SNew(SBox)
											.MinDesiredWidth(130.0f)
											[
												SAssignNew(GroupComboBox, SComboBox< TSharedPtr<FString> >)
												.Visibility(this, &SAutomationWindow::HandleGroupsVisibility)
												.OptionsSource(&GroupComboList)
												.InitiallySelectedItem(GroupComboList[0])
												.OnGenerateWidget(this, &SAutomationWindow::GenerateGroupComboItem)
												.OnSelectionChanged(this, &SAutomationWindow::HandleGroupChanged)
												.ContentPadding(FMargin(4.0, 1.0f))
												[
													SNew(STextBlock)
													.Text(this, &SAutomationWindow::GetGroupComboText)
												]
											]
										]

										+ SHorizontalBox::Slot()
										.FillWidth(0.02f)
										[
											SNew(SSpacer)
										]

										+ SHorizontalBox::Slot()
										.FillWidth(0.8f)
										.VAlign(VAlign_Center)
										.Padding(2.0f, 0, 0, 0)
										[
											SAssignNew(AutomationSearchBox, SSearchBox)
											.ToolTipText(LOCTEXT("Search Tests", "Search Tests"))
											.OnTextChanged(this, &SAutomationWindow::OnFilterTextChanged)
											.IsEnabled(this, &SAutomationWindow::IsAutomationControllerIdle)
										]

										+ SHorizontalBox::Slot()
										.FillWidth(0.02f)
										[
											SNew(SSpacer)
										]

										+SHorizontalBox::Slot()
										.VAlign(VAlign_Center)
										.AutoWidth()
										[
											SNew(STextBlock)
											.Text( LOCTEXT("AutomationFilterLabel", "Filter:") )
										]

										+SHorizontalBox::Slot()
										.AutoWidth()
										.HAlign(HAlign_Left)
										.VAlign(VAlign_Center)
										[
											SAutomationWindow::MakeAutomationFilterToolBar(AutomationWindowActions.ToSharedRef(), SharedThis(this))
										]
									]

									+ SVerticalBox::Slot()
									.FillHeight(1.0f)
									[
										//the actual table full of tests
										TestTable.ToSharedRef()
									]
								]
							]
						]

						+ SOverlay::Slot()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(SThrobber)	
							.Visibility(this, &SAutomationWindow::GetTestsUpdatingThrobberVisibility)
						]
					]
				]

				+ SSplitter::Slot()
				.Value(0.33f)
				[

						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						[
							SNew(SOverlay)
							+ SOverlay::Slot()
							[
								SNew(SBox)
									.Visibility(this, &SAutomationWindow::GetTestGraphVisibility)
									[
										//Graphical Results Panel
										SNew( SVerticalBox )

										+ SVerticalBox::Slot()
											.AutoHeight()
											[
												SNew(SHorizontalBox)

												+ SHorizontalBox::Slot()
													.HAlign(HAlign_Left)
													[
														SNew(STextBlock)
															.Text( LOCTEXT("AutomationTest_GraphicalResults", "Automation Test Graphical Results:"))
													]

												+ SHorizontalBox::Slot()
													.HAlign(HAlign_Right)
													.AutoWidth()
													[
														SNew(STextBlock)
															.Text( LOCTEXT("AutomationTest_Display", "Display:"))
													]

												+ SHorizontalBox::Slot()
													.HAlign(HAlign_Right)
													.AutoWidth()
													[
														SNew(SCheckBox)
															.Style(FCoreStyle::Get(), "RadioButton")
															.IsChecked(this, &SAutomationWindow::HandleResultDisplayTypeIsChecked, EAutomationGrapicalDisplayType::DisplayName)
															.OnCheckStateChanged(this, &SAutomationWindow::HandleResultDisplayTypeStateChanged, EAutomationGrapicalDisplayType::DisplayName)
															[
																SNew(STextBlock)
																	.Text( LOCTEXT("AutomationTest_GraphicalResultsDisplayName", "Name"))
															]
													]

												+ SHorizontalBox::Slot()
													.HAlign(HAlign_Right)
													.AutoWidth()
													[
														SNew(SCheckBox)
															.Style(FCoreStyle::Get(), "RadioButton")
															.IsChecked(this, &SAutomationWindow::HandleResultDisplayTypeIsChecked, EAutomationGrapicalDisplayType::DisplayTime)
															.OnCheckStateChanged(this, &SAutomationWindow::HandleResultDisplayTypeStateChanged, EAutomationGrapicalDisplayType::DisplayTime)
															[
																SNew(STextBlock)
																	.Text( LOCTEXT("AutomationTest_GraphicalResultsDisplayTime", "Time"))
															]
													]
											]

										+ SVerticalBox::Slot()
											.FillHeight(1.0f)
											[
												SNew(SBorder)
												[
													SNew(SScrollBox)

													+ SScrollBox::Slot()
														[
															SAssignNew(GraphicalResultBox, SAutomationGraphicalResultBox, InAutomationController)
														]
												]
											]
									]
							]

							+ SOverlay::Slot()
							[
								SNew(SBox)
								.Visibility(this, &SAutomationWindow::GetTestLogVisibility)
								[
									//results panel
									SNew( SVerticalBox )
									+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew( STextBlock )
										.Text( LOCTEXT("AutomationTest_Results", "Automation Test Results:") )
									]

									+ SVerticalBox::Slot()
									.FillHeight(1.0f)
									.Padding(0.0f, 4.0f, 0.0f, 0.0f)
									[
										//list of results for the selected test
										SNew(SBorder)
										.BorderImage(FAutomationWindowStyle::Get().GetBrush("Brushes.Panel"))
										[
											SNew(SScrollBox)
											.Orientation(EOrientation::Orient_Horizontal)
											+SScrollBox::Slot()
											.FillSize(1.0f)
											[
												SAssignNew(LogListView, SListView<TSharedPtr<FAutomationOutputMessage> >)
												.ListItemsSource(&LogMessages)
												.SelectionMode(ESelectionMode::Multi)
												.OnGenerateRow(this, &SAutomationWindow::OnGenerateWidgetForLog)
												.OnSelectionChanged(this, &SAutomationWindow::HandleLogListSelectionChanged)
											]
										]
									]
								]
							]
						]
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							SNew(SBorder)
							.BorderImage(FAutomationWindowStyle::Get().GetBrush("ToolPanel.GroupBorder"))
							.Padding(FMargin(8.0f, 6.0f))
							[
								// Add the command bar
								SAssignNew(CommandBar, SAutomationWindowCommandBar, NotificationList)
								.OnCopyLogClicked(this, &SAutomationWindow::HandleCommandBarCopyLogClicked)
							]
						]
				]
			]

		+ SOverlay::Slot()
			.HAlign( HAlign_Center )
			.VAlign( VAlign_Center )
			.Padding( 15.0f )
			[
				NotificationList
			]

		+ SOverlay::Slot()
			.HAlign( HAlign_Center )
			.VAlign( VAlign_Center )
			.Padding( 15.0f )
			[
				SNew(SBorder)
					.BorderImage(FAutomationWindowStyle::Get().GetBrush("NotificationList.ItemBackground"))
					.Padding(8.0f)
					.Visibility(this, &SAutomationWindow::HandleSelectSessionOverlayVisibility)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("SelectSessionOverlayText", "Please select at least one instance from the Session Browser"))
					]
			]
	];

	SessionManager->OnCanSelectSession().AddSP( this, &SAutomationWindow::HandleSessionManagerCanSelectSession );
	SessionManager->OnSelectedSessionChanged().AddSP( this, &SAutomationWindow::HandleSessionManagerSelectionChanged );
	SessionManager->OnSessionInstanceUpdated().AddSP( this, &SAutomationWindow::HandleSessionManagerInstanceChanged );

	FindWorkers();

	FAutomationTestFramework::Get().LoadTestTagMappings();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SAutomationWindow::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	UAutomationControllerSettings* Settings = UAutomationControllerSettings::StaticClass()->GetDefaultObject<UAutomationControllerSettings>();
	if (AutomationController->GetFilteredReports().Num() > 0 && Settings && Settings->bOpenTagEditorWhenInitialSelectedTestsNames)
	{
		Settings->bOpenTagEditorWhenInitialSelectedTestsNames = false;

		AutomationController->SetEnabledTests(Settings->InitialSelectedTestsNames);

		SAutomationTestTagEditor::ShowInWindow(this->AsShared(), AutomationController->GetEnabledReports(), FOnTestTagsSavedEvent());

		AutomationController->SetEnabledTests(TArray<FString>());
		Settings->InitialSelectedTestsNames.Empty();
	}
}


void SAutomationWindow::HandleResultDisplayTypeStateChanged( ECheckBoxState NewRadioState, EAutomationGrapicalDisplayType::Type NewDisplayType)
{
	if (NewRadioState == ECheckBoxState::Checked)
	{
		GraphicalResultBox->SetDisplayType(NewDisplayType);
	}
}

ECheckBoxState SAutomationWindow::HandleResultDisplayTypeIsChecked( EAutomationGrapicalDisplayType::Type InDisplayType ) const
{
	return (GraphicalResultBox->GetDisplayType() == InDisplayType)
		? ECheckBoxState::Checked
		: ECheckBoxState::Unchecked;
}

const FSlateBrush* SAutomationWindow::GetTestBackgroundBorderImage() const
{
	switch(TestBackgroundType)
	{
	case EAutomationTestBackgroundStyle::Game:
		return FAutomationWindowStyle::Get().GetBrush("AutomationWindow.GameGroupBorder");

	case EAutomationTestBackgroundStyle::Editor:
		return FAutomationWindowStyle::Get().GetBrush("AutomationWindow.EditorGroupBorder");

	case EAutomationTestBackgroundStyle::Unknown:
	default:
		return FAutomationWindowStyle::Get().GetBrush("ToolPanel.GroupBorder");
	}
}

void SAutomationWindow::CreateCommands()
{
	check(!AutomationWindowActions.IsValid());
	AutomationWindowActions = MakeShareable(new FUICommandList);

	const FAutomationWindowCommands& Commands = FAutomationWindowCommands::Get();
	FUICommandList& ActionList = *AutomationWindowActions;

	ActionList.MapAction( Commands.RefreshTests,
		FExecuteAction::CreateRaw( this, &SAutomationWindow::ListTests ),
		FCanExecuteAction::CreateRaw( this, &SAutomationWindow::IsAutomationControllerIdle )
		);

	ActionList.MapAction( Commands.FindWorkers,
		FExecuteAction::CreateRaw( this, &SAutomationWindow::FindWorkers ),
		FCanExecuteAction::CreateRaw( this, &SAutomationWindow::IsAutomationControllerIdle )
		);

	ActionList.MapAction( Commands.ErrorFilter,
		FExecuteAction::CreateRaw( this, &SAutomationWindow::OnToggleErrorFilter ),
		FCanExecuteAction::CreateRaw( this, &SAutomationWindow::IsAutomationControllerIdle ),
		FIsActionChecked::CreateRaw( this, &SAutomationWindow::IsErrorFilterOn )
		);

	ActionList.MapAction( Commands.WarningFilter,
		FExecuteAction::CreateRaw( this, &SAutomationWindow::OnToggleWarningFilter ),
		FCanExecuteAction::CreateRaw( this, &SAutomationWindow::IsAutomationControllerIdle ),
		FIsActionChecked::CreateRaw( this, &SAutomationWindow::IsWarningFilterOn )
		);

	ActionList.MapAction( Commands.DeveloperDirectoryContent,
		FExecuteAction::CreateRaw( this, &SAutomationWindow::OnToggleDeveloperDirectoryIncluded ),
		FCanExecuteAction::CreateRaw( this, &SAutomationWindow::IsAutomationControllerIdle ),
		FIsActionChecked::CreateRaw( this, &SAutomationWindow::IsDeveloperDirectoryIncluded )
		);

	ActionList.MapAction( Commands.ExcludedTestsFilter,
		FExecuteAction::CreateRaw( this, &SAutomationWindow::OnToggleExcludedTestsFilter ),
		FCanExecuteAction::CreateRaw( this, &SAutomationWindow::IsAutomationControllerIdle ),
		FIsActionChecked::CreateRaw( this, &SAutomationWindow::IsExcludedTestsFilterOn )
		);

	ActionList.MapAction( Commands.EditTags,
		FExecuteAction::CreateRaw(this, &SAutomationWindow::HandleEditTagsButtonClicked ),
		FCanExecuteAction::CreateRaw(this, &SAutomationWindow::IsEditTagsButtonEnabled )
		);

	// Added button for running the currently open level test.
#if WITH_EDITOR
	ActionList.MapAction(Commands.RunLevelTest,
		FExecuteAction::CreateRaw(this, &SAutomationWindow::OnRunLevelTest),
		FCanExecuteAction::CreateRaw(this, &SAutomationWindow::CanExecuteRunLevelTest)
		);
#endif // WITH_EDITOR

	ActionList.MapAction(Commands.CreateNewPreset,
		FExecuteAction::CreateRaw(this, &SAutomationWindow::OnNewPresetClicked),
		FCanExecuteAction::CreateRaw(this, &SAutomationWindow::IsAutomationControllerIdle)
	);

	ActionList.MapAction(Commands.RemoveCurrentPreset,
		FExecuteAction::CreateRaw(this, &SAutomationWindow::OnRemovePresetClicked),
		FCanExecuteAction::CreateRaw(this, &SAutomationWindow::IsAutomationControllerIdle)
	);

	ActionList.MapAction(Commands.RenameCurrentPreset,
		FExecuteAction::CreateRaw(this, &SAutomationWindow::OnRenamePresetClicked),
		FCanExecuteAction::CreateRaw(this, &SAutomationWindow::IsAutomationControllerIdle)
	);
}

TSharedRef< SWidget > SAutomationWindow::MakeAutomationWindowToolBar( const TSharedRef<FUICommandList>& InCommandList, TSharedPtr<class SAutomationWindow> InAutomationWindow )
{
	return InAutomationWindow->MakeAutomationWindowToolBar(InCommandList);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef< SWidget > SAutomationWindow::MakeAutomationWindowToolBar( const TSharedRef<FUICommandList>& InCommandList )
{
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, TSharedRef<SWidget> RunTests, TSharedRef<SWidget> DeviceGroups, TSharedRef<SWidget> PresetBox, TSharedRef<SWidget> TagFilter, TWeakPtr<class SAutomationWindow> InAutomationWindow)
		{
			ToolbarBuilder.BeginSection("Automation");
			{
				ToolbarBuilder.AddWidget(RunTests);

				// Added button for running the currently open level test.
#if WITH_EDITOR
				ToolbarBuilder.AddToolBarButton(
					FAutomationWindowCommands::Get().RunLevelTest,
					NAME_None,
					TAttribute<FText>(),
					LOCTEXT("RunLevelTest_ToolTip", "If the currently loaded editor level is a test map, click this to select the test and run it immediately."),
					FSlateIcon(FAutomationWindowStyle::Get().GetStyleSetName(), "AutomationWindow.RunLevelTest"));
#endif

				ToolbarBuilder.AddWidget(SNew(SSpacer));
				ToolbarBuilder.AddToolBarButton( FAutomationWindowCommands::Get().RefreshTests );
				ToolbarBuilder.AddWidget(SNew(SSpacer));
				ToolbarBuilder.AddToolBarButton( FAutomationWindowCommands::Get().FindWorkers );
				ToolbarBuilder.AddWidget(SNew(SSpacer));
			}
			ToolbarBuilder.EndSection();
			ToolbarBuilder.BeginSection("GroupFlags");
			{
				ToolbarBuilder.AddWidget(SNew(SSpacer));
				ToolbarBuilder.AddWidget(DeviceGroups);
				ToolbarBuilder.AddWidget(SNew(SSpacer));
			}
			ToolbarBuilder.EndSection();
			ToolbarBuilder.BeginSection("Presets");
			{
				ToolbarBuilder.AddWidget(SNew(SSpacer));
				ToolbarBuilder.AddWidget( PresetBox );
				ToolbarBuilder.AddWidget(SNew(SSpacer));
			}
			ToolbarBuilder.EndSection();
			ToolbarBuilder.BeginSection("Tags");
			{
				ToolbarBuilder.AddWidget(SNew(SSpacer));
				ToolbarBuilder.AddToolBarButton(FAutomationWindowCommands::Get().EditTags);
				ToolbarBuilder.AddWidget(SNew(SSpacer));
				ToolbarBuilder.AddWidget( TagFilter );
				ToolbarBuilder.AddWidget(SNew(SSpacer));
			}
			ToolbarBuilder.EndSection();
		}
	};

	TSharedRef<SWidget> RunTests = 
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign( VAlign_Center )
		.AutoWidth()
		.Padding(0)
		[
			SNew( SComboButton )
			.ComboButtonStyle( FAppStyle::Get(), "SimpleComboButton" )
			.OnGetMenuContent( this, &SAutomationWindow::GenerateTestsOptionsMenuContent )
			.ToolTipText( LOCTEXT("TestOptionsToolTip", "Test Options") )
			.HasDownArrow(true)
			.ContentPadding(0)
			.ButtonContent()
			[

			SNew( SButton )
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ToolTipText( LOCTEXT( "StartStop Tests", "Start / Stop tests" ) )
			.OnClicked( this, &SAutomationWindow::RunTests )
			.IsEnabled( this, &SAutomationWindow::IsAutomationRunButtonEnabled )
			.HAlign( HAlign_Center )
			.VAlign( VAlign_Center )
			.ContentPadding(0)
			[
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.VAlign( VAlign_Center )
				[
					SNew( SVerticalBox )
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( SOverlay )
						+SOverlay::Slot()
						.Padding(0)
						.VAlign( VAlign_Center )
						.HAlign( HAlign_Center )
						[
							SNew( SImage ) 
							.Image( this, &SAutomationWindow::GetRunAutomationIcon )
						]
						+SOverlay::Slot()
						.Padding(0)
						.VAlign( VAlign_Bottom )
						.HAlign( HAlign_Right )
						[
							SNew( STextBlock )
							.Margin( FMargin(22.0f, 25.0f, 0.0, 0.0f) )
							.Text( this, &SAutomationWindow::OnGetNumEnabledTestsString )
							.ColorAndOpacity(FSlateColor::UseForeground())
							.MinDesiredWidth(55.0f)
							.Justification(ETextJustify::Right)
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign( HAlign_Center )
					[
						SNew( STextBlock )
						.Visibility( bIsLabelVisibilityEnabled ? EVisibility::Visible : EVisibility::Collapsed )
						.Text( this, &SAutomationWindow::GetRunAutomationLabel )
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]

			]
		];

	TSharedRef<SWidget> DeviceGroups =
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.FillHeight(1)
		[
			SNew(SSimpleComboButton)
			.OnGetMenuContent(this, &SAutomationWindow::GenerateGroupOptionsMenuContent)
			.Icon(FAutomationWindowStyle::Get().GetBrush(FName("AutomationWindow.GroupSettings" + GetSmallIconExtension())))
			.Text(LOCTEXT("GroupOptions_Label", "Device Groups"))
			.ToolTipText(LOCTEXT("GroupOptionsToolTip", "Device Group Options"))
			.HasDownArrow(true)
		];

	TSharedRef<SWidget> TestPresets = 
		SNew( SVerticalBox )
		+SVerticalBox::Slot()
		.FillHeight(1)
		.VAlign( VAlign_Center )
		.Padding( 5.0f )
		[
			SNew( SHorizontalBox )
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew( SOverlay )
				//Preset Combo / Text
				+SOverlay::Slot()
				[
					SNew( SHorizontalBox )
					+ SHorizontalBox::Slot()
					.VAlign( VAlign_Center )
					.FillWidth( 65.0f )
					[
						SNew( STextBlock )
						.Text( LOCTEXT("AutomationPresetLabel", "Preset:") )
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew( SComboButton )
						.OnGetMenuContent( this, &SAutomationWindow::GeneratePresetsMenuContent )
						.ButtonContent()
						[
							SNew( SHorizontalBox )
							+ SHorizontalBox::Slot()
							.FillWidth(1)
							.VAlign( VAlign_Center )
							[
								SAssignNew( PresetComboButtonText, STextBlock )
								.MinDesiredWidth( 60.0f )
								.Text( LOCTEXT("AutomationPreset_None", "None") )
							]
						]
					]
				]
				+SOverlay::Slot()
				[
					SNew( SHorizontalBox )
					.Visibility( this, &SAutomationWindow::HandlePresetTextVisibility )
					+SHorizontalBox::Slot()
					.FillWidth( 100.0f )
					[
						SAssignNew( PresetTextBox, SEditableTextBox )
						.OnTextCommitted( this, &SAutomationWindow::HandlePresetTextCommited )
						.IsEnabled( this, &SAutomationWindow::IsAutomationControllerIdle )
					]
				]
			]

			//Save button
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew( SButton )
				.ButtonStyle(FAutomationWindowStyle::Get(), "NoBorder" )
				.OnClicked( this, &SAutomationWindow::HandleSavePresetClicked )
				.ToolTipText( LOCTEXT("AutomationPresetSaveButtonTooltip", "Save the current test list") )
				.IsEnabled(this, &SAutomationWindow::IsSaveButtonEnabled)
				.Content()
				[
					SNew( SImage )
					.Image(FAutomationWindowStyle::Get().GetBrush("Icons.Save"))
				]
			]
		];

	TagFilter = SNew(SAutomationTestTagFilter, AutomationTagFilter);
	TagFilter->SetOnTestFilterAppliedEvent(FOnTestFilterAppliedEvent::CreateLambda([this]()
		{
			OnRefreshTestCallback();
		}));

	FToolBarBuilder ToolbarBuilder( InCommandList, FMultiBoxCustomization::None );
	ToolbarBuilder.SetLabelVisibility(bIsLabelVisibilityEnabled ? EVisibility::Visible : EVisibility::Collapsed);
	TWeakPtr<SAutomationWindow> AutomationWindow = SharedThis(this);
	Local::FillToolbar(ToolbarBuilder, RunTests, DeviceGroups, TestPresets, TagFilter.ToSharedRef(), AutomationWindow);

	// Create the tool bar!
	return
		SNew( SHorizontalBox )
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew( SBorder )
			.Padding(0)
			.BorderImage(FAutomationWindowStyle::Get().GetBrush("NoBorder") )
			.IsEnabled( FSlateApplication::Get().GetNormalExecutionAttribute() )
			[
				ToolbarBuilder.MakeWidget()
			]
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


TSharedRef< SWidget > SAutomationWindow::MakeAutomationFilterToolBar(const TSharedRef<FUICommandList>& InCommandList, TSharedPtr<class SAutomationWindow> InAutomationWindow)
{
	return InAutomationWindow->MakeAutomationFilterToolBar(InCommandList);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef< SWidget > SAutomationWindow::MakeAutomationFilterToolBar(const TSharedRef<FUICommandList>& InCommandList)
{
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.BeginSection("Filters");
			{
				ToolbarBuilder.AddWidget(SNew(SSpacer));
				ToolbarBuilder.AddToolBarButton(FAutomationWindowCommands::Get().ErrorFilter);
				ToolbarBuilder.AddToolBarButton(FAutomationWindowCommands::Get().WarningFilter);
				ToolbarBuilder.AddToolBarButton(FAutomationWindowCommands::Get().ExcludedTestsFilter);
				ToolbarBuilder.AddToolBarButton(FAutomationWindowCommands::Get().DeveloperDirectoryContent);
				ToolbarBuilder.AddWidget(SNew(SSpacer));
			}
			ToolbarBuilder.EndSection();
		}
	};

	FToolBarBuilder ToolbarBuilder(InCommandList, FMultiBoxCustomization::None);
	ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);
	TWeakPtr<SAutomationWindow> AutomationWindow = SharedThis(this);
	Local::FillToolbar(ToolbarBuilder);

	// Create the tool bar!
	return
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBorder)
			.Padding(0)
			.BorderImage(FAutomationWindowStyle::Get().GetBrush("NoBorder"))
			.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
			[
				ToolbarBuilder.MakeWidget()
			]
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

EVisibility SAutomationWindow::HandlePresetComboVisibility( ) const
{
	return bAddingTestPreset ? EVisibility::Hidden : EVisibility::Visible;
}

EVisibility SAutomationWindow::HandlePresetTextVisibility( ) const
{
	return bAddingTestPreset || bRenamingTestPreset ? EVisibility::Visible : EVisibility::Hidden;
}

EVisibility SAutomationWindow::HandleGroupsVisibility() const
{
	return GroupComboList.Num() == 1 ? EVisibility::Collapsed : EVisibility::Visible;
}

bool SAutomationWindow::IsSaveButtonEnabled() const
{
	return (!bAddingTestPreset && !bRenamingTestPreset && SelectedPreset.IsValid() && IsAutomationControllerIdle());
}

void SAutomationWindow::HandlePresetTextCommited( const FText& CommittedText, ETextCommit::Type CommitType )
{
	if( CommitType == ETextCommit::OnEnter )
	{
		if ( bAddingTestPreset )
		{
			bAddingTestPreset = false;
			if (CommittedText.IsEmpty())
			{
				return;
			}

			TArray<FString> EnabledTests;
			AutomationController->GetEnabledTestNames(EnabledTests);
			AutomationPresetPtr NewPreset = TestPresetManager->AddNewPreset(CommittedText, EnabledTests);
			if (NewPreset.IsValid())
			{
				SelectedPreset = NewPreset;

				PresetTextBox->SetText(FText());
			}
		}
		else if (bRenamingTestPreset)
		{
			bRenamingTestPreset = false;
			if (CommittedText.IsEmpty())
			{
				return;
			}

			SelectedPreset->SetName(CommittedText);
			PresetTextBox->SetText(FText());
		}

		PresetComboButtonText->SetText(CommittedText);
	}
	else if( CommitType == ETextCommit::OnCleared || CommitType == ETextCommit::OnUserMovedFocus )
	{
		if( bAddingTestPreset || bRenamingTestPreset )
		{
			bAddingTestPreset = false;
			bRenamingTestPreset = false;
			SelectedPreset = nullptr;
			PresetTextBox->SetText(FText());
		}
	}
}

void SAutomationWindow::HandleRequesteFilterChanged(TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo)
{
	const int32 EntryIndex = RequestedFilterComboList.Find(Item);
	EAutomationTestFlags NewRequestedFlags = EAutomationTestFlags::SmokeFilter;

	switch (EntryIndex)
	{
		case 0:	//	"All Tests"
			NewRequestedFlags = EAutomationTestFlags_FilterMask;
			break;
		case 1:	//	"Smoke Tests"
			NewRequestedFlags = EAutomationTestFlags::SmokeFilter;
			break;
		case 2:	//	"Engine Tests"
			NewRequestedFlags = EAutomationTestFlags::EngineFilter;
			break;
		case 3:	//	"Product Tests"
			NewRequestedFlags = EAutomationTestFlags::ProductFilter;
			break;
		case 4:	//	"Performance Tests"
			NewRequestedFlags = EAutomationTestFlags::PerfFilter;
			break;
		case 5:	//	"Stress Tests"
			NewRequestedFlags = EAutomationTestFlags::StressFilter;
			break;
		case 6:	//	"Standard Tests"
			NewRequestedFlags = EAutomationTestFlags::SmokeFilter | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ProductFilter | EAutomationTestFlags::PerfFilter;
			break;
		case 7: //  "Negative Tests"
			NewRequestedFlags = EAutomationTestFlags::NegativeFilter;
			break;
	}
	AutomationController->SetRequestedTestFlags(NewRequestedFlags);
}

void SAutomationWindow::HandleGroupChanged(TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo)
{
	AutomationGroupFilter->SetFilters(GroupFiltersMap[*Item.Get()]);
	OnRefreshTestCallback();
}

void SAutomationWindow::ExpandEnabledTests( TSharedPtr< IAutomationReport > InReport )
{
	// Expand node if the report is enabled or contains an enabled test
	TestTable->SetItemExpansion( InReport, InReport->IsEnabled() || InReport->GetEnabledTestsNum() > 0 );

	// Iterate through the child nodes to see if they should be expanded
	TArray<TSharedPtr< IAutomationReport > > Reports = InReport->GetFilteredChildren();

	for ( int32 ChildItem = 0; ChildItem < Reports.Num(); ChildItem++ )
	{
		ExpandEnabledTests( Reports[ ChildItem ] );
	}
}

FReply SAutomationWindow::HandleSavePresetClicked()
{
	FText PresetSaveStatusText;
	bool bSavedSuccessfully = false;
	if (SelectedPreset.IsValid())
	{
		TArray<FString> EnabledTests;
		AutomationController->GetEnabledTestNames(EnabledTests);
		SelectedPreset->SetEnabledTests(EnabledTests);
		bSavedSuccessfully = TestPresetManager->SavePreset(SelectedPreset.ToSharedRef());
		if (bSavedSuccessfully)
		{
			PresetSaveStatusText = FText::Format(LOCTEXT("AutomationPresetSaveSucceeded", "Preset '{0}' saved successfully."), SelectedPreset->GetName());
		}
		else
		{
			PresetSaveStatusText = FText::Format(LOCTEXT("AutomationPresetSaveFailed", "Failed to save preset '{0}'."), SelectedPreset->GetName());
		}
	}
	const FNotificationInfo SaveNotificationInfo = FNotificationInfo(PresetSaveStatusText);
	TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(SaveNotificationInfo);
	if (NotificationPtr)
	{
		NotificationPtr->SetCompletionState(bSavedSuccessfully ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail);
	}
	return FReply::Handled();
}

void SAutomationWindow::OnNewPresetClicked()
{
	bAddingTestPreset = true;
	FSlateApplication::Get().SetUserFocus(0, PresetTextBox.ToSharedRef(), EFocusCause::SetDirectly);
}

void SAutomationWindow::OnRemovePresetClicked()
{
	if ( SelectedPreset.IsValid() )
	{
		TestPresetManager->RemovePreset(SelectedPreset.ToSharedRef());
		SelectedPreset = nullptr;
		PresetComboButtonText->SetText(LOCTEXT("AutomationPresetComboLabel", "None"));
	}
}

void SAutomationWindow::OnRenamePresetClicked()
{
	if ( SelectedPreset.IsValid() )
	{
		bRenamingTestPreset = true;
		PresetTextBox->SetText(SelectedPreset->GetName());
		FSlateApplication::Get().SetUserFocus(0, PresetTextBox.ToSharedRef(), EFocusCause::SetDirectly);
	}
}

FText SAutomationWindow::GetPresetComboText() const
{
	if ( SelectedPreset.IsValid() )
	{
		return SelectedPreset->GetName();
	}
	else
	{
		return LOCTEXT("AutomationPresetComboLabel", "None");
	}
}

FText SAutomationWindow::GetRequestedFilterComboText() const
{
	if (RequestedFilterComboBox->GetSelectedItem().IsValid())
	{
		return FText::FromString(*RequestedFilterComboBox->GetSelectedItem());
	}
	else
	{
		return LOCTEXT("AutomationRequestedFilterComboLabel", "All Tests");
	}
}

FText SAutomationWindow::GetGroupComboText() const
{
	if (GroupComboBox->GetSelectedItem().IsValid())
	{
		return FText::FromString(*GroupComboBox->GetSelectedItem());
	}
	else
	{
		return LOCTEXT("AutomationGroupComboLabel", "All Groups");
	}
}

TSharedRef<SWidget> SAutomationWindow::GenerateRequestedFilterComboItem(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*InItem));
}

TSharedRef<SWidget> SAutomationWindow::GenerateGroupComboItem(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*InItem));
}

TSharedRef< SWidget > SAutomationWindow::GenerateGroupOptionsMenuContent( TWeakPtr<class SAutomationWindow> InAutomationWindow )
{
	TSharedPtr<SAutomationWindow> AutomationWindow(InAutomationWindow.Pin());
	if( AutomationWindow.IsValid() )
	{
		return AutomationWindow->GenerateGroupOptionsMenuContent();
	}

	//Return empty menu
	FMenuBuilder MenuBuilder( true, nullptr );
	MenuBuilder.BeginSection("AutomationWindowGroupOptions", LOCTEXT("DeviceGroupOptions", "Device Group Options"));
	MenuBuilder.EndSection();
	return MenuBuilder.MakeWidget();
}

TSharedRef< SWidget > SAutomationWindow::GenerateGroupOptionsMenuContent( )
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, AutomationWindowActions );
	const uint32 NumFlags = EAutomationDeviceGroupTypes::Max;
	TSharedPtr<SWidget> FlagWidgets[NumFlags];
	for( int32 i=0; i<NumFlags; i++ )
	{
		FlagWidgets[i] = 
			SNew(SCheckBox)
			.IsChecked(this, &SAutomationWindow::IsDeviceGroupCheckBoxIsChecked, i)
			.OnCheckStateChanged(this, &SAutomationWindow::HandleDeviceGroupCheckStateChanged, i)
			.Padding(FMargin(4.0f, 0.0f))
			.ToolTipText(EAutomationDeviceGroupTypes::ToDescription((EAutomationDeviceGroupTypes::Type)i))
			.IsEnabled( this, &SAutomationWindow::IsAutomationControllerIdle )
			.Content()
			[
				SNew(STextBlock)
				.Text(EAutomationDeviceGroupTypes::ToName((EAutomationDeviceGroupTypes::Type)i))
			];
	}

	MenuBuilder.BeginSection("AutomationWindowGroupDevices", LOCTEXT("GroupTypeOptions", "Group Types"));
	{
		for( int32 i=0; i<NumFlags;i++ )
		{
			MenuBuilder.AddWidget(FlagWidgets[i].ToSharedRef(),FText::GetEmpty());
		}
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef< SWidget > SAutomationWindow::GeneratePresetsMenuContent( TWeakPtr<class SAutomationWindow> InAutomationWindow )
{
	TSharedPtr<SAutomationWindow> AutomationWindow(InAutomationWindow.Pin());
	if (AutomationWindow.IsValid())
	{
		return AutomationWindow->GenerateGroupOptionsMenuContent();
	}

	//Return empty menu
	FMenuBuilder MenuBuilder(true, nullptr);
	return MenuBuilder.MakeWidget();
}

TSharedRef< SWidget > SAutomationWindow::GeneratePresetsMenuContent()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, AutomationWindowActions);
	MenuBuilder.BeginSection("AutomationWindowNewPresetGroup", LOCTEXT("GroupNewPreset", "New"));
	MenuBuilder.AddMenuEntry(FAutomationWindowCommands::Get().CreateNewPreset);
	MenuBuilder.EndSection();
	const TArray<AutomationPresetPtr>& Presets = TestPresetManager->GetAllPresets();
	TSharedPtr<TArray<TSharedPtr<SCheckBox>>> CheckBoxesSP = MakeShared<TArray<TSharedPtr<SCheckBox>>>();

	MenuBuilder.BeginSection("AutomationWindowCurrentPresetGroup", LOCTEXT("GroupCurrentPreset", "Current Preset"));
	{
		MenuBuilder.AddMenuEntry(FAutomationWindowCommands::Get().RenameCurrentPreset);
		MenuBuilder.AddMenuEntry(FAutomationWindowCommands::Get().RemoveCurrentPreset);

		
		for (int32 i = 0; i < Presets.Num(); i++)
		{
			const auto& preset = Presets[i];
			TSharedPtr<SCheckBox> CheckBox =
				SNew(SCheckBox)
				.Style(FAutomationWindowStyle::Get(), "AutomationWindow.ToggleButton")
				.IsChecked(preset.IsValid() && SelectedPreset.IsValid() && preset->GetID() == SelectedPreset->GetID())
				.OnCheckStateChanged(this, &SAutomationWindow::HandlePresetCheckStateChanged, i, CheckBoxesSP)
				.Padding(FMargin(4.0f, 0.0f))
				.IsEnabled(this, &SAutomationWindow::IsAutomationControllerIdle)
				.Content()
				[
					SNew(STextBlock)
					.Text(preset.IsValid() ? preset.ToSharedRef()->GetName() : LOCTEXT("AutomationPreset_None", "None"))
				];
			MenuBuilder.AddWidget(CheckBox.ToSharedRef(), FText::GetEmpty());
			CheckBoxesSP->Add(CheckBox);
		}
	}

	return MenuBuilder.MakeWidget();
}

/** Returns if full size screen shots are enabled */
ECheckBoxState SAutomationWindow::IsDeviceGroupCheckBoxIsChecked(const int32 DeviceGroupFlag) const
{
	return AutomationController->IsDeviceGroupFlagSet((EAutomationDeviceGroupTypes::Type)DeviceGroupFlag) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}
/** Toggles if we are collecting full size screenshots */
void SAutomationWindow::HandleDeviceGroupCheckStateChanged(ECheckBoxState CheckBoxState, const int32 DeviceGroupFlag)
{
	//Update the device groups
	AutomationController->ToggleDeviceGroupFlag((EAutomationDeviceGroupTypes::Type)DeviceGroupFlag);
	AutomationController->UpdateDeviceGroups();
	
	//Update header
	RebuildPlatformIcons();

	//Need to force the tree to do a full refresh here because the reports have changed but the tree will keep using cached data.
	TestTable->RebuildList();
}

void SAutomationWindow::HandlePresetCheckStateChanged(ECheckBoxState CheckBoxState, const int32 EntryIndex, TSharedPtr<TArray<TSharedPtr<SCheckBox>>> CheckBoxes)
{
	if (CheckBoxState != ECheckBoxState::Checked)
	{
		SelectedPreset = nullptr;
		PresetComboButtonText->SetText(LOCTEXT("AutomationPreset_None", "None"));
		return;
	}
	for (int32 i = 0; i < CheckBoxes->Num(); i++)
	{
		if (EntryIndex != i)
		{
			(*CheckBoxes)[i].Get()->SetIsChecked(false);
		}
	}

	SelectedPreset = *(&TestPresetManager->GetAllPresets()[EntryIndex]);
	PresetComboButtonText->SetText(SelectedPreset.IsValid() ? SelectedPreset.Get()->GetName() : LOCTEXT("AutomationPreset_None", "None"));

	if (SelectedPreset.IsValid())
	{
		AutomationController->SetEnabledTests(SelectedPreset->GetEnabledTests());
		TestTable->RequestTreeRefresh();

		//Expand selected items
		TestTable->ClearExpandedItems();
		TArray< TSharedPtr< IAutomationReport > >& TestReports = AutomationController->GetFilteredReports();
		for (int32 Index = 0; Index < TestReports.Num(); Index++)
		{
			ExpandEnabledTests(TestReports[Index]);
		}
	}
}

TSharedRef< SWidget > SAutomationWindow::GenerateTestsOptionsMenuContent( TWeakPtr<class SAutomationWindow> InAutomationWindow )
{
	TSharedPtr<SAutomationWindow> AutomationWindow(InAutomationWindow.Pin());
	if( AutomationWindow.IsValid() )
	{
		return AutomationWindow->GenerateTestsOptionsMenuContent();
	}

	//Return empty menu
	FMenuBuilder MenuBuilder( true, nullptr );
	MenuBuilder.BeginSection("AutomationWindowRunTest", LOCTEXT("RunTestOptions", "Advanced Settings"));
	MenuBuilder.EndSection();
	return MenuBuilder.MakeWidget();
}

TSharedRef< SWidget > SAutomationWindow::GenerateTestsOptionsMenuContent( )
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, AutomationWindowActions );
	TSharedRef<SWidget> NumTests = 
		SNew(SBox)
		.WidthOverride( 200.0f )
		[
			SNew( SHorizontalBox )
			+SHorizontalBox::Slot()
			.Padding(0.0f,0.0f,4.0f, 0.0f)
			.AutoWidth()
			[
				SNew( STextBlock )
				.Text( LOCTEXT("NumTestsToolTip", "Number of runs:") )
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SSpinBox<int32>)
				.MinValue(1)
				.MaxValue(1000)
				.MinSliderValue(1)
				.MaxSliderValue(1000)
				.Value(this,&SAutomationWindow::GetRepeatCount)
				.OnValueChanged(this,&SAutomationWindow::OnChangeRepeatCount)
				.IsEnabled( this, &SAutomationWindow::IsAutomationControllerIdle )
			]

		];

	
	TSharedRef<SWidget> SendAnalyticsWidget =
		SNew(SCheckBox)
		.IsChecked(this, &SAutomationWindow::IsSendAnalyticsCheckBoxChecked)
		.OnCheckStateChanged(this, &SAutomationWindow::HandleSendAnalyticsBoxCheckStateChanged)
		.Padding(FMargin(4.0f, 0.0f))
		.ToolTipText(LOCTEXT("AutomationSendAnalyticsTip", "If checked, tests send analytics results to the backend"))
		.IsEnabled(this, &SAutomationWindow::IsAutomationControllerIdle)
		.Content()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AutomationSendAnalyticsText", "Enable analytics"))
		];


	TSharedRef<SWidget> KeepPIEOpenWidget =
		SNew(SCheckBox)
		.IsChecked(this, &SAutomationWindow::KeepPIEOpenCheckBoxChecked)
		.OnCheckStateChanged(this, &SAutomationWindow::HandleKeepPIEOpenBoxCheckStateChanged)
		.Padding(FMargin(4.0f, 0.0f))
		.ToolTipText(LOCTEXT("AutomationKeepPIEOpenTip", "If checked, the PIE will be kept open at the end of the test pass"))
		.IsEnabled(this, &SAutomationWindow::IsAutomationControllerIdle)
		.Content()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AutomationKeepPIEOpenText", "Keep PIE open at the end"))
		];


	TSharedRef<SWidget> AutoExpandSingleItemSubgroupsWidget =
		SNew(SCheckBox)
		.IsChecked(this, &SAutomationWindow::AutoExpandSingleItemSubgroupsCheckBoxChecked)
		.OnCheckStateChanged(this, &SAutomationWindow::HandleAutoExpandSingleItemSubgroupsCheckStateChanged)
		.Padding(FMargin(4.0f, 0.0f))
		.ToolTipText(LOCTEXT("AutomationAutoExpandSingleItemSubgroupsTip", "If checked, automatic expansion of single-item test subgroups will be enabled"))
		.IsEnabled(this, &SAutomationWindow::IsAutomationControllerIdle)
		.Content()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AutomationAutoExpandSingleItemSubgroupsText", "Auto expand single-item subgroups"))
		];


	MenuBuilder.BeginSection("AutomationWindowRunTest", LOCTEXT("RunTestOptions", "Advanced Settings"));
	{
		MenuBuilder.AddWidget(NumTests, FText::GetEmpty());
		MenuBuilder.AddWidget(SendAnalyticsWidget, FText::GetEmpty());
		#if WITH_EDITOR
		MenuBuilder.AddWidget(KeepPIEOpenWidget, FText::GetEmpty());
		#endif //WITH_EDITOR
		MenuBuilder.AddWidget(AutoExpandSingleItemSubgroupsWidget, FText::GetEmpty());
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

ECheckBoxState SAutomationWindow::IsSendAnalyticsCheckBoxChecked() const
{
	return AutomationController->IsSendAnalytics() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SAutomationWindow::HandleSendAnalyticsBoxCheckStateChanged(ECheckBoxState CheckBoxState)
{
	AutomationController->SetSendAnalytics(CheckBoxState == ECheckBoxState::Checked);
}

ECheckBoxState SAutomationWindow::KeepPIEOpenCheckBoxChecked() const
{
	return AutomationController->KeepPIEOpen() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SAutomationWindow::HandleKeepPIEOpenBoxCheckStateChanged(ECheckBoxState CheckBoxState)
{
	AutomationController->SetKeepPIEOpen(CheckBoxState == ECheckBoxState::Checked);
}

/** Returns if we should automatically expand single-item test subgroups */
ECheckBoxState SAutomationWindow::AutoExpandSingleItemSubgroupsCheckBoxChecked() const
{
	return bAutoExpandSingleItemSubgroups ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

/** Toggles if automatic expansion of single-item subgroups is enabled */
void SAutomationWindow::HandleAutoExpandSingleItemSubgroupsCheckStateChanged(ECheckBoxState CheckBoxState)
{
	bAutoExpandSingleItemSubgroups = (CheckBoxState == ECheckBoxState::Checked);
}

TArray<FString> SAutomationWindow::SaveExpandedTestNames(TSet<TSharedPtr<IAutomationReport>> ExpandedItems)
{
	TArray<FString> ExpandedItemsNames;
	for ( TSharedPtr<IAutomationReport> ExpandedItem : ExpandedItems )
	{
		ExpandedItemsNames.Add(ExpandedItem->GetDisplayNameWithDecoration());
	}
	return ExpandedItemsNames;
}

// Expanded the given item if its name is in the array of strings given.
void SAutomationWindow::ExpandItemsInList(TSharedPtr<STreeView<TSharedPtr<IAutomationReport>>> InTestTable, TSharedPtr<IAutomationReport> InReport, TArray<FString> ItemsToExpand)
{
	InTestTable->SetItemExpansion(InReport, ItemsToExpand.Contains(InReport->GetDisplayNameWithDecoration()));

	TArray<TSharedPtr<IAutomationReport>> ChildReports = InReport->GetFilteredChildren();

	for ( int32 Index = 0; Index < ChildReports.Num(); Index++ )
	{
		ExpandItemsInList(InTestTable, ChildReports[Index], ItemsToExpand);
	}
}

// Only valid in the editor
#if WITH_EDITOR
TSharedPtr<SWidget> SAutomationWindow::HandleAutomationListContextMenuOpening()
{
	TArray< TSharedPtr<IAutomationReport> >SelectedReport = TestTable->GetSelectedItems();

	TArray<FString> TestNames;
	TArray<FString> AssetNames;
	for (TSharedPtr<IAutomationReport> Report : SelectedReport)
	{
		if (Report.IsValid())
		{
			FString TestName = Report->GetFullTestPath();
			if (!TestName.IsEmpty())
			{
				TestNames.Add(MoveTemp(TestName));
			}
			FString Param = Report->GetTestParameter();
			if (Param.StartsWith(TEXT("/")))
			{
				// Assume that if parameter start with a "/", it should be an asset
				AssetNames.Add(MoveTemp(Param));
			}
		}
	}		
	
	if (AssetNames.Num() || TestNames.Num())
	{
		return SNew(SAutomationTestItemContextMenu, AssetNames, TestNames);
	}

	return nullptr;
}

void SAutomationWindow::RunSelectedTests()
{
	AutomationController->SetVisibleTestsEnabled(false);
	SetAllSelectedTestsChecked(true);
	RunTests();
}

namespace
{
	/**
	 * Kind of a hack - this requires that we know we group all the map tests coming from blueprints under "Functional Tests"
	 */
	TSharedPtr<IAutomationReport> GetFunctionalTestsReport(const TArray< TSharedPtr< IAutomationReport > >& TestReports)
	{
		for ( auto& Report : TestReports )
		{
			if ( Report->GetDisplayName() == TEXT("Functional Tests") )
			{
				return Report;
			}

			auto FoundInChild = GetFunctionalTestsReport(Report->GetChildReports());
			if ( FoundInChild.IsValid() )
			{
				return FoundInChild;
			}
		}
		return TSharedPtr<IAutomationReport>();
	}

	void FindReportByGameRelativeAssetPath(const TSharedPtr<IAutomationReport>& RootReport, const FString& AssetRelativePath, TArray<TSharedPtr<IAutomationReport>>& OutLevelReports)
	{
		FString TestAssetRelativePath(RootReport->GetTestParameter());

		if ( TestAssetRelativePath.StartsWith(AssetRelativePath) )
		{
			OutLevelReports.Add(RootReport);
		}
		else
		{
			// Branch node
			for ( auto ChildReport : RootReport->GetChildReports() )
			{
				FindReportByGameRelativeAssetPath(ChildReport, AssetRelativePath, OutLevelReports);
			}
		}
	}
} // namespace

void SAutomationWindow::FindTestReportsForCurrentEditorLevel(TArray<TSharedPtr<IAutomationReport>>& OutLevelReports)
{
	// Find the current map path
	if ( GWorld && GWorld->GetCurrentLevel() && GWorld->GetCurrentLevel()->GetPackage())
	{
		const FString MapPath = GWorld->GetCurrentLevel()->GetPackage()->GetPathName();
		if (!MapPath.IsEmpty())
		{
			auto FunctionTestsReport = GetFunctionalTestsReport(AutomationController->GetFilteredReports());
			if (FunctionTestsReport.IsValid())
			{
				FindReportByGameRelativeAssetPath(FunctionTestsReport, MapPath, OutLevelReports);
			}
		}
	}
}

bool SAutomationWindow::CanExecuteRunLevelTest()
{
	return IsAutomationControllerIdle();
}

void SAutomationWindow::OnRunLevelTest()
{
	TArray<TSharedPtr<IAutomationReport>> LevelReports;
	FindTestReportsForCurrentEditorLevel(LevelReports);

	if ( LevelReports.Num() > 0 )
	{
		TestTable->ClearSelection();
		for ( auto& LevelReport : LevelReports )
		{
			TestTable->SetItemSelection(LevelReport, true);
		}

		ScrollToTest(LevelReports[0]);
		RunSelectedTests();
	}
}

void SAutomationWindow::ScrollToTest(TSharedPtr<IAutomationReport> InReport)
{
	auto& RootReports = AutomationController->GetFilteredReports();
	for ( auto ChildReport : RootReports )
	{
		auto ShouldExpand = ExpandToTest(ChildReport, InReport);
		TestTable->SetItemExpansion(ChildReport, ShouldExpand);
	}

	TestTable->RequestScrollIntoView(InReport);
}

bool SAutomationWindow::ExpandToTest(TSharedPtr<IAutomationReport> InRoot, TSharedPtr<IAutomationReport> InReport)
{
	if ( InRoot == InReport )
		return true;

	bool WasExpanded = false;

	for ( auto ChildReport : InRoot->GetChildReports() )
	{
		auto ShouldExpand = ExpandToTest(ChildReport, InReport);
		TestTable->SetItemExpansion(ChildReport, ShouldExpand);

		if ( ShouldExpand )
		{
			// Here we could just return true, but we want to collapse all the other reports 
			// so we keep going and just remember that we found the test.
			WasExpanded = true;
		}
	}

	return WasExpanded;
}

#endif


void SAutomationWindow::PopulateReportSearchStrings( const TSharedPtr< IAutomationReport >& Report, OUT TArray< FString >& OutSearchStrings ) const
{
	if( !Report.IsValid() )
	{
		return;
	}

	OutSearchStrings.Add( Report->GetDisplayName() );
	OutSearchStrings.Add( Report->GetFullTestPath() );
}

void SAutomationWindow::OnExpansionChanged(TSharedPtr<IAutomationReport> InItem, bool bExpanded)
{
	ExpandSingleItemSubgroups(InItem, bExpanded);
}

void SAutomationWindow::ExpandSingleItemSubgroups(TSharedPtr<IAutomationReport> InItem, bool bExpanded)
{
	check(InItem.IsValid());
	if (bAutoExpandSingleItemSubgroups && bExpanded)
	{
		TArray<TSharedPtr<IAutomationReport>>& FilteredChildren = InItem->GetFilteredChildren();
		if (FilteredChildren.Num() == 1)
		{
			TSharedPtr<IAutomationReport> SingleChild = FilteredChildren.Top();
			check(SingleChild.IsValid());

			if (!SingleChild->IsParent())
			{
				return;
			}

			TestTable->SetItemExpansion(InItem, bExpanded);

			TArray<TSharedPtr<IAutomationReport>>& SingleChildFilteredChildren = SingleChild->GetFilteredChildren();
			static auto IsChildPredicate = [](const TSharedPtr<IAutomationReport>& InItemToCheck) -> bool
			{
				check(InItemToCheck.IsValid());
				return (!InItemToCheck->IsParent());
			};

			const bool bSingleChildHasAtLeastOneChildLeaf = (nullptr != SingleChildFilteredChildren.FindByPredicate(IsChildPredicate));
			if (bSingleChildHasAtLeastOneChildLeaf)
			{
				return;
			}

			ExpandSingleItemSubgroups(SingleChild, bExpanded);
		}
	}
}

void SAutomationWindow::OnGetChildren(TSharedPtr<IAutomationReport> InItem, TArray<TSharedPtr<IAutomationReport> >& OutItems)
{
	OutItems = InItem->GetFilteredChildren();
}

void SAutomationWindow::OnTestExpansionRecursive(TSharedPtr<IAutomationReport> InAutomationReport, bool bInIsItemExpanded)
{
	if ( InAutomationReport.IsValid() )
	{
		TArray<TSharedPtr<IAutomationReport> >& FilteredChildren = InAutomationReport->GetFilteredChildren();

		TestTable->SetItemExpansion(InAutomationReport, bInIsItemExpanded);

		for ( TSharedPtr<IAutomationReport>& Child : FilteredChildren )
		{
			OnTestExpansionRecursive(Child, bInIsItemExpanded);
		}
	}
}

void SAutomationWindow::OnTestSelectionChanged(TSharedPtr<IAutomationReport> Selection, ESelectInfo::Type /*SelectInfo*/)
{
	TSharedPtr<IAutomationReport> PreviousSelectionLock = PreviousSelection.Pin();
	if ( PreviousSelectionLock.IsValid() )
	{
		PreviousSelectionLock->OnSetResults.Unbind();
	}

	bHasChildTestSelected = false;

	UpdateTestLog(Selection);

	if ( Selection.IsValid() )
	{
		Selection->OnSetResults.BindRaw(this, &SAutomationWindow::UpdateTestLog);
		PreviousSelection = Selection;

		if ( Selection->GetTotalNumChildren() == 0 )
		{
			bHasChildTestSelected = true;
		}
	}

	CommandBar->SetCopyButtonVisibility(GetTestLogVisibility());
}

void SAutomationWindow::UpdateTestLog(TSharedPtr<IAutomationReport> Selection)
{
	//empty the previous log
	LogMessages.Empty();

	if (Selection.IsValid())
	{
		//accumulate results for each device cluster that supports the test
		int32 NumClusters = AutomationController->GetNumDeviceClusters();
		for (int32 ClusterIndex = 0; ClusterIndex < NumClusters; ++ClusterIndex)
		{
			//no sense displaying device name if only one is available
			if (NumClusters > 1)
			{
				FString DeviceTypeName = AutomationController->GetClusterGroupName(ClusterIndex) + TEXT("  -  ") + Selection->GetGameInstanceName(ClusterIndex);
				LogMessages.Add(MakeShareable(new FAutomationOutputMessage(DeviceTypeName, TEXT("Automation.Header"))));
			}

			const int32 NumOfPasses = Selection->GetNumResults(ClusterIndex);
			for( int32 PassIndex = 0; PassIndex < NumOfPasses; ++PassIndex )
			{
				//get strings out of the report and populate the Log Messages
				FAutomationTestResults TestResults = Selection->GetResults(ClusterIndex,PassIndex);

				//no sense displaying device name if only one is available
				if (NumOfPasses > 1)
				{
					FString PassHeader = LOCTEXT("TestPassHeader", "Pass:").ToString();
					PassHeader += FString::Printf(TEXT("%i"),PassIndex+1);
					LogMessages.Add(MakeShareable(new FAutomationOutputMessage(PassHeader, TEXT("Automation.Header"))));
				}

				for (const FAutomationExecutionEntry& Entry : TestResults.GetEntries())
				{
					switch (Entry.Event.Type)
					{
					case EAutomationEventType::Info:
						LogMessages.Add(MakeShareable(new FAutomationOutputMessage(Entry.ToString(), TEXT("Automation.Normal"))));
						break;
					case EAutomationEventType::Warning:
						LogMessages.Add(MakeShareable(new FAutomationOutputMessage(Entry.ToString(), TEXT("Automation.Warning"))));
						break;
					case EAutomationEventType::Error:
						LogMessages.Add(MakeShareable(new FAutomationOutputMessage(Entry.ToString(), TEXT("Automation.Error"))));
						break;
					}
				}

				if ( ( TestResults.GetWarningTotal() == 0 ) && ( TestResults.GetErrorTotal() == 0 ) && ( Selection->GetState(ClusterIndex, PassIndex) == EAutomationState::Success ) )
				{
					LogMessages.Add(MakeShareable(new FAutomationOutputMessage(LOCTEXT("AutomationTest_SuccessMessage", "Success").ToString(), TEXT("Automation.Normal"))));
				}

				LogMessages.Add(MakeShareable(new FAutomationOutputMessage(TEXT(""), TEXT("Log.Normal"))));
			}
		}
	}

	//rebuild UI
	LogListView->RequestListRefresh();
}


EVisibility SAutomationWindow::GetTestLogVisibility( ) const
{
	return (GetTestGraphVisibility() == EVisibility::Visible) ? EVisibility::Hidden : EVisibility::Visible;
}


EVisibility SAutomationWindow::GetTestGraphVisibility( ) const
{
	//Show the graphical window if we don't have a child test selected and we have results to view
	return (!bHasChildTestSelected && GraphicalResultBox->HasResults()) ? EVisibility::Visible : EVisibility::Hidden;
}


void SAutomationWindow::HeaderCheckboxStateChange(ECheckBoxState InCheckboxState)
{
	const bool bState = (InCheckboxState == ECheckBoxState::Checked)? true : false;

	AutomationController->SetVisibleTestsEnabled(bState);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SAutomationWindow::RebuildPlatformIcons()
{
	//empty header UI
	PlatformsHBox->ClearChildren();

	//for each device type
	int32 NumClusters = AutomationController->GetNumDeviceClusters();
	for (int32 ClusterIndex = 0; ClusterIndex < NumClusters; ++ClusterIndex)
	{
		PlatformsHBox->AddSlot()
		.AutoWidth()
		.MaxWidth(ColumnWidth)
		[
			SNew(STextBlock)
			.Text(FText::Format(
				LOCTEXT("GameInstances_Header", "Instances ({0})"),
				FText::AsNumber( AutomationController->GetNumDevicesInCluster(ClusterIndex) )
			))
			.ToolTipText(CreateDeviceTooltip(ClusterIndex))
		];
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FText SAutomationWindow::CreateDeviceTooltip(int32 ClusterIndex)
{
	FTextBuilder ReportBuilder;

	const int32 NumClusters = AutomationController->GetNumDeviceClusters();
	if( NumClusters > 1 )
	{
		ReportBuilder.AppendLine(LOCTEXT("ToolTipClusterName", "Cluster Name:"));
		ReportBuilder.AppendLine(AutomationController->GetClusterGroupName(ClusterIndex));
	}

	ReportBuilder.AppendLine(LOCTEXT("ToolTipGameInstances", "Game Instances:"));

	int32 NumDevices = AutomationController->GetNumDevicesInCluster( ClusterIndex );
	for ( int32 DeviceIndex = 0; DeviceIndex < NumDevices; ++DeviceIndex )
	{
		ReportBuilder.AppendLine(AutomationController->GetGameInstanceName(ClusterIndex, DeviceIndex).LeftPad(2));
	}

	return ReportBuilder.ToText();
}


void SAutomationWindow::ClearAutomationUI ()
{
	// Clear results from the automation controller
	AutomationController->ClearAutomationReports();
	TestTable->RequestTreeRefresh();

	// Clear the platform icons
	if (PlatformsHBox.IsValid())
	{
		PlatformsHBox->ClearChildren();
	}
	
	// Clear the log
	LogMessages.Empty();
	LogListView->RequestListRefresh();
}


TSharedRef<ITableRow> SAutomationWindow::OnGenerateWidgetForTest( TSharedPtr<IAutomationReport> InItem, const TSharedRef<STableViewBase>& OwnerTable )
{
	bIsRequestingTests = false;
	return SNew(SAutomationTestItem, OwnerTable)
		.TestStatus(InItem)
		.ColumnWidth(ColumnWidth)
		.IsLocalSession(ActiveSession->IsStandalone())
		.HighlightText(this, &SAutomationWindow::HandleAutomationHighlightText)
		.OnCheckedStateChanged(this, &SAutomationWindow::HandleItemCheckBoxCheckedStateChanged);
}


TSharedRef<ITableRow> SAutomationWindow::OnGenerateWidgetForLog(TSharedPtr<FAutomationOutputMessage> Message, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(Message.IsValid());

	// \[((?:[\w]\:|\\)(?:(?:\\[A-Za-z_\-\s0-9\.]+)+)\.(?:cpp|h|ini))\((\d+)\)\]$
	// https://regex101.com/r/vV4cV7/25
	FRegexPattern FileAndLinePattern(TEXT("\\[((?:[\\w]\\:|\\\\)(?:(?:\\\\[A-Za-z_\\-\\s0-9\\.]+)+)\\.(?:cpp|h|ini))\\((\\d+)\\)\\]$"));
	FRegexMatcher FileAndLineRegexMatcher(FileAndLinePattern, Message->Text);

	TSharedRef<SWidget> SourceLink = SNullWidget::NullWidget;

	FString MessageString = Message->Text;

	if ( FileAndLineRegexMatcher.FindNext() )
	{
		FString FileName = FileAndLineRegexMatcher.GetCaptureGroup(1);
		int32 LineNumber = FCString::Atoi(*FileAndLineRegexMatcher.GetCaptureGroup(2));

		// Remove the hyperlink from the message, since we're splitting it into its own string.
		MessageString.LeftChopInline(FileAndLineRegexMatcher.GetCaptureGroup(0).Len(), EAllowShrinking::No);

		SourceLink = SNew(SHyperlink)
			.Style(FAutomationWindowStyle::Get(), "Common.GotoNativeCodeHyperlink")
			.TextStyle(FAutomationWindowStyle::Get(), Message->Style)
			.OnNavigate_Lambda([=] { FSlateApplication::Get().GotoLineInSource(FileName, LineNumber); })
			.Text(FText::FromString(FileAndLineRegexMatcher.GetCaptureGroup(0)));
	}

	return SNew(STableRow<TSharedPtr<FAutomationOutputMessage> >, OwnerTable)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0)
			[
				SNew(STextBlock)
				.TextStyle(FAutomationWindowStyle::Get(), Message->Style )
				.Text(FText::FromString(MessageString))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0)
			[
				SourceLink
			]
		];
}


FText SAutomationWindow::OnGetNumEnabledTestsString() const
{
	int32 NumPasses = AutomationController->GetNumPasses();
	if( NumPasses > 1 )
	{
		return FText::Format(LOCTEXT("NumEnabledTestsFmt", "{0} x{1}"), FText::AsNumber(AutomationController->GetEnabledTestsNum()), FText::AsNumber(NumPasses));
	}
	else
	{
		return FText::AsNumber(AutomationController->GetEnabledTestsNum());
	}
}


FText SAutomationWindow::OnGetNumDevicesInClusterString(const int32 ClusterIndex) const
{
	return FText::AsNumber(AutomationController->GetNumDevicesInCluster(ClusterIndex));
}

void SAutomationWindow::OnRefreshTestCallback()
{
	//if the window hasn't been created yet
	if (!PlatformsHBox.IsValid())
	{
		return;
	}

	//rebuild the platform header
	RebuildPlatformIcons();

	//filter the tests that are shown
	AutomationController->SetFilter( AutomationFilters );

	// Only expand the child nodes if we have a text filter
	bool ExpandChildren = !AutomationTextFilter->GetRawFilterText().IsEmpty();

	TArray< TSharedPtr< IAutomationReport > >& TestReports = AutomationController->GetFilteredReports();

	for( int32 Index = 0; Index < TestReports.Num(); Index++ )
	{
		ExpandTreeView( TestReports[ Index ], ExpandChildren );

		// Expand any items that where expanded before refresh tests was pressed.
		if( !ExpandChildren )
		{
			ExpandItemsInList( TestTable, TestReports[Index], SavedExpandedItems );
		}
	}

	SavedExpandedItems.Empty();

	//rebuild the UI
	TestTable->RequestTreeRefresh();

	//update the background style
	UpdateTestListBackgroundStyle();
}


void SAutomationWindow::OnTestAvailableCallback( EAutomationControllerModuleState::Type InAutomationControllerState )
{
	AutomationControllerState = InAutomationControllerState;

	// Only list tests on opening the Window if the asset registry isn't in the middle of loading tests.
	if ( InAutomationControllerState == EAutomationControllerModuleState::Ready && AutomationController->GetFilteredReports().Num() == 0 && !bIsRequestingTests)
	{
#if WITH_EDITOR
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		if ( !AssetRegistryModule.Get().IsLoadingAssets() )
		{
			ListTests();
		}
#else
		ListTests();
#endif
	}
}

void SAutomationWindow::OnTestsCompleteCallback()
{
	// Simulate selection again after testing finishes.
	if ( TestTable->GetNumItemsSelected() > 0 )
	{
		OnTestSelectionChanged(TestTable->GetSelectedItems()[0], ESelectInfo::Direct);
	}
}


void SAutomationWindow::ExpandTreeView( TSharedPtr< IAutomationReport > InReport, const bool ShouldExpand )
{
	// Expand node if the report is flagged
	TestTable->SetItemExpansion( InReport, ShouldExpand && InReport->ExpandInUI() );

	// Iterate through the child nodes to see if they should be expanded
	TArray<TSharedPtr< IAutomationReport > > Reports = InReport->GetFilteredChildren();

	for ( int32 ChildItem = 0; ChildItem < Reports.Num(); ChildItem++ )
	{
		ExpandTreeView( Reports[ ChildItem ], ShouldExpand );
	}
}

//TODO AUTOMATION - remove
/** Updates list of all the tests */
void SAutomationWindow::ListTests( )
{
	// Save Expanded state prior to refresh
	TSet<TSharedPtr<IAutomationReport>> ExpandedItems;
	TestTable->GetExpandedItems(ExpandedItems);
	SavedExpandedItems = SaveExpandedTestNames(ExpandedItems);

	AutomationController->RequestTests();
}


//TODO AUTOMATION - remove
/** Finds available workers */
void SAutomationWindow::FindWorkers()
{
	ActiveSession = SessionManager->GetSelectedSession();

	bool SessionIsValid = ActiveSession.IsValid() && (ActiveSession->GetSessionOwner() == FPlatformProcess::UserName(false));

	if (SessionIsValid)
	{
		bIsRequestingTests = true;

		AutomationController->RequestAvailableWorkers(ActiveSession->GetSessionId());

		RebuildPlatformIcons();
	}
	else
	{
		bIsRequestingTests = false;
		// Clear UI if the session is invalid
		ClearAutomationUI();
	}

	MenuBar->SetEnabled( SessionIsValid );
}

void SAutomationWindow::HandleSessionManagerInstanceChanged()
{
	UpdateTestListBackgroundStyle();
}

void SAutomationWindow::UpdateTestListBackgroundStyle()
{
	TArray<TSharedPtr<ISessionInstanceInfo>> OutInstances;

	if( ActiveSession.IsValid() )
	{
		ActiveSession->GetInstances(OutInstances);
	}

	TestBackgroundType = EAutomationTestBackgroundStyle::Unknown;

	if( OutInstances.Num() > 0 )
	{
		FString FirstInstanceType = OutInstances[0]->GetInstanceType();

		if( FirstInstanceType.Contains(TEXT("Editor")) )
		{
			TestBackgroundType = EAutomationTestBackgroundStyle::Editor;
		}
		else if( FirstInstanceType.Contains(TEXT("Game")) )
		{
			TestBackgroundType = EAutomationTestBackgroundStyle::Game;
		}
	}
}


FReply SAutomationWindow::RunTests()
{
	if( AutomationControllerState == EAutomationControllerModuleState::Running )
	{
		AutomationController->StopTests();
	}
	else
	{
		// Prompt to save current map when running a test. 
#if WITH_EDITOR
		if ( !GIsDemoMode )
		{
			// If there are any unsaved changes to the current level, see if the user wants to save those first.
			const bool bPromptUserToSave = true;
			const bool bSaveMapPackages = true;
			const bool bSaveContentPackages = true;
			if ( FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages) == false )
			{
				// something went wrong or the user pressed cancel.  Return to the editor so the user doesn't lose their changes		
				return FReply::Handled();
			}
		}
#endif

		AutomationController->RunTests( ActiveSession->IsStandalone() );
	}

	LogMessages.Empty();
	LogListView->RequestListRefresh();

	//Clear old results
	GraphicalResultBox->ClearResults();

	return FReply::Handled();
}


/** Filtering */
void SAutomationWindow::OnFilterTextChanged( const FText& InFilterText )
{
	AutomationTextFilter->SetRawFilterText( InFilterText );
	AutomationSearchBox->SetError( AutomationTextFilter->GetFilterErrorText() );

	//update the widget
	OnRefreshTestCallback();
}


bool SAutomationWindow::IsDeveloperDirectoryIncluded() const
{
	return AutomationController->IsDeveloperDirectoryIncluded();
}


void SAutomationWindow::OnToggleDeveloperDirectoryIncluded()
{
	//Change controller filter
	AutomationController->SetDeveloperDirectoryIncluded(!IsDeveloperDirectoryIncluded());
	// need to call this to request update
	ListTests();
}

bool SAutomationWindow::IsExcludedTestsFilterOn() const
{
	return AutomationGeneralFilter->ShouldShowOnlyExcludedTests();
}


void SAutomationWindow::OnToggleExcludedTestsFilter()
{
	AutomationGeneralFilter->SetShowOnlyExcludedTests(!IsExcludedTestsFilterOn());
	OnRefreshTestCallback();
}


bool SAutomationWindow::IsSmokeTestFilterOn() const
{
	return AutomationGeneralFilter->OnlyShowSmokeTests();
}


void SAutomationWindow::OnToggleSmokeTestFilter()
{
	AutomationGeneralFilter->SetOnlyShowSmokeTests( !IsSmokeTestFilterOn() );
	OnRefreshTestCallback();
}


bool SAutomationWindow::IsWarningFilterOn() const
{
	return AutomationGeneralFilter->ShouldShowWarnings();
}


void SAutomationWindow::OnToggleWarningFilter()
{
	AutomationGeneralFilter->SetShowWarnings( !IsWarningFilterOn() );
	OnRefreshTestCallback();
}


bool SAutomationWindow::IsErrorFilterOn() const
{
	return AutomationGeneralFilter->ShouldShowErrors();
}


void SAutomationWindow::OnToggleErrorFilter()
{
	AutomationGeneralFilter->SetShowErrors( !IsErrorFilterOn() );
	OnRefreshTestCallback();
}


void SAutomationWindow::OnChangeRepeatCount(int32 InNewValue)
{
	AutomationController->SetNumPasses(InNewValue);
}

int32 SAutomationWindow::GetRepeatCount() const
{
	return AutomationController->GetNumPasses();
}


FString SAutomationWindow::GetSmallIconExtension() const
{
	FString Brush;
	if (FMultiBoxSettings::UseSmallToolBarIcons.Get())
	{
		Brush += TEXT( ".Small" );
	}
	return Brush;
}


EVisibility SAutomationWindow::GetLargeToolBarVisibility() const
{
	return FMultiBoxSettings::UseSmallToolBarIcons.Get() ? EVisibility::Collapsed : EVisibility::Visible;
}


const FSlateBrush* SAutomationWindow::GetRunAutomationIcon() const
{
	FString Brush = TEXT( "AutomationWindow" );
	if( AutomationControllerState == EAutomationControllerModuleState::Running )
	{
		Brush += TEXT( ".StopTests" );	// Temporary brush type for stop tests
	}
	else
	{
		Brush += TEXT( ".RunTests" );
	}
	Brush += GetSmallIconExtension();
	return FAutomationWindowStyle::Get().GetBrush( *Brush );
}


FText SAutomationWindow::GetRunAutomationLabel() const
{
	if( AutomationControllerState == EAutomationControllerModuleState::Running )
	{
		return LOCTEXT( "RunStopTestsLabel", "Stop Tests" );
	}
	else
	{
		return LOCTEXT( "RunStartTestsLabel", "Start Tests" );
	}
}


FText SAutomationWindow::HandleAutomationHighlightText( ) const
{
	if ( AutomationSearchBox.IsValid() )
	{
		return AutomationSearchBox->GetText();
	}
	return FText();
}


EVisibility SAutomationWindow::HandleSelectSessionOverlayVisibility( ) const
{
	if (SessionManager->GetSelectedInstances().Num() > 0)
	{
		return EVisibility::Hidden;
	}

	return EVisibility::Visible;
}


void SAutomationWindow::HandleSessionManagerCanSelectSession( const TSharedPtr<ISessionInfo>& Session, bool& CanSelect )
{
	// We are using a compilation condition here as FMessageDialog::Open will show dialog only if GIsEditor && !IsRunningCommandlet() && FCoreDelegates::ModalErrorMessage.IsBound()
	// else it always gives us EAppReturnType::No answer in EAppMsgType::YesNo case.
	// The result is the sessions cannot be chosen in not editor case (ie: UnrealFrontEnd)   
#if WITH_EDITOR 
	if (ActiveSession.IsValid() && AutomationController->CheckTestResultsAvailable())
	{
		EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("ChangeSessionDialog", "Are you sure you want to change sessions?\nAll automation results data will be lost"));
		CanSelect = Result == EAppReturnType::Yes ? true : false;
	}
#endif //WITH_EDITOR
}


void SAutomationWindow::HandleSessionManagerSelectionChanged( const TSharedPtr<ISessionInfo>& SelectedSession )
{
	if (!GIsAutomationTesting)
	{
		FindWorkers();
	}
}


bool SAutomationWindow::IsAutomationControllerIdle() const
{
	return AutomationControllerState != EAutomationControllerModuleState::Running;
}


bool SAutomationWindow::IsAutomationRunButtonEnabled() const
{
	return AutomationControllerState != EAutomationControllerModuleState::Disabled;
}


void SAutomationWindow::CopyLog( )
{
	TArray<TSharedPtr<FAutomationOutputMessage> > SelectedItems = LogListView->GetSelectedItems();

	if (SelectedItems.Num() > 0)
	{
		FString SelectedText;

		for( int32 Index = 0; Index < SelectedItems.Num(); ++Index )
		{
			SelectedText += SelectedItems[Index]->Text;
			SelectedText += LINE_TERMINATOR;
		}

		FPlatformApplicationMisc::ClipboardCopy( *SelectedText );
	}
}


FReply SAutomationWindow::HandleCommandBarCopyLogClicked( )
{
	CopyLog();

	return FReply::Handled();
}


void SAutomationWindow::HandleLogListSelectionChanged( TSharedPtr<FAutomationOutputMessage> InItem, ESelectInfo::Type SelectInfo )
{
	CommandBar->SetNumLogMessages(LogListView->GetNumItemsSelected());
}


void SAutomationWindow::ChangeTheSelectionToThisRow(TSharedPtr< IAutomationReport >  ThisRow) 
{
	TestTable->SetSelection(ThisRow, ESelectInfo::Direct);
}

bool SAutomationWindow::IsRowSelected(TSharedPtr< IAutomationReport >  ThisRow)
{
	TArray< TSharedPtr<IAutomationReport> >SelectedReport = TestTable->GetSelectedItems();

	bool ThisRowIsInTheSelectedSet = false;

	for (int i = 0; i<SelectedReport.Num();++i)
	{
		if (SelectedReport[i] == ThisRow)
		{
			ThisRowIsInTheSelectedSet = true;
		}
	}
	return ThisRowIsInTheSelectedSet;
}


void SAutomationWindow::SetAllSelectedTestsChecked( bool InChecked )
{
	TArray< TSharedPtr<IAutomationReport> >SelectedReport = TestTable->GetSelectedItems();

	for (int i = 0; i<SelectedReport.Num();++i)
	{
		if (SelectedReport[i].IsValid())
		{
			SelectedReport[i]->SetEnabled(InChecked);
		}
	}
}


bool SAutomationWindow::IsAnySelectedRowEnabled()
{
	TArray< TSharedPtr<IAutomationReport> >SelectedReport = TestTable->GetSelectedItems();

	//Do check or uncheck selected rows based on current settings
	bool bFoundCheckedRow = false;
	bool bFoundNotCheckedRow = false;
	bool bRowCheckedValue = true;

	//Check all the rows if there is a mixture of checked and unchecked then we set all checked, otherwise set to opposite of current values

	for (int i = 0; i<SelectedReport.Num();++i)
	{
		if (SelectedReport[i].IsValid())
		{
			if (SelectedReport[i]->IsEnabled())
			{
				bFoundCheckedRow = true;
			}
			else
			{
				bFoundNotCheckedRow = true;
			}
		}
		//break when all rows checked or different values found
		if (bFoundCheckedRow && bFoundNotCheckedRow)
		{
			break;
		}
	}

	//if rows were all checked set to unchecked otherwise we can set to checked
	if (bFoundCheckedRow && !bFoundNotCheckedRow)
	{
		bRowCheckedValue = false;
	}

	return bRowCheckedValue;
}


/* SWidget implementation
 *****************************************************************************/

FReply SAutomationWindow::OnKeyUp( const FGeometry& InGeometry, const FKeyEvent& InKeyEvent )
{
	if (InKeyEvent.GetKey() == EKeys::SpaceBar)
	{
		SetAllSelectedTestsChecked(IsAnySelectedRowEnabled());
		return FReply::Handled();
	}
	return FReply::Unhandled();
}


FReply SAutomationWindow::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if (InKeyEvent.IsControlDown())
	{
		if (InKeyEvent.GetKey() == EKeys::C)
		{
			CopyLog();

			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}


/* SAutomationWindow callbacks
 *****************************************************************************/

void SAutomationWindow::HandleItemCheckBoxCheckedStateChanged( TSharedPtr< IAutomationReport > TestStatus )
{
	//If multiple rows selected then handle all the rows
	if (AreMultipleRowsSelected())
	{
		//if current row is not in the selected list select that row
		if(IsRowSelected(TestStatus))
		{
			//Just set them all to the opposite of the row just clicked.
			SetAllSelectedTestsChecked(!TestStatus->IsEnabled());
		}
		else
		{
			//Change the selection to this row rather than keep other rows selected unrelated to the ticked/unticked item
			ChangeTheSelectionToThisRow(TestStatus);
			TestStatus->SetEnabled( !TestStatus->IsEnabled() );
		}
	}
	else
	{
		TestStatus->SetEnabled( !TestStatus->IsEnabled() );
	}
}


bool SAutomationWindow::HandleItemCheckBoxIsEnabled( ) const
{
	return IsAutomationControllerIdle();
}


bool SAutomationWindow::HandleMainContentIsEnabled() const
{
	return (SessionManager->GetSelectedInstances().Num() > 0);
}

void SAutomationWindow::HandleEditTagsButtonClicked() const
{
#if WITH_EDITOR
	SAutomationTestTagEditor::ShowInWindow(this->AsShared(), AutomationController->GetEnabledReports(),
		FOnTestTagsSavedEvent::CreateLambda([this]()
			{
				AutomationController->OnTestsRefreshed().Broadcast();
				TagFilter->ClearFilter();
			}));
#endif
}

bool SAutomationWindow::IsEditTagsButtonEnabled() const
{
	return AutomationController->GetEnabledReports().Num() > 0;
}

#if WITH_EDITOR
// React to asset registry finishing updating.
// We only want to do this if there are no tests already listed, otherwise this fires every time you save a map for example.
void SAutomationWindow::OnAssetRegistryFilesLoaded()
{
	ListTests();
}
#endif

#undef LOCTEXT_NAMESPACE
