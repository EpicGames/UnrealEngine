// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "IAutomationReport.h"
#include "ISessionManager.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Input/SComboBox.h"
#include "SAutomationGraphicalResultBox.h"
#include "Widgets/Views/STreeView.h"

#if WITH_EDITOR
#include "AssetRegistry/IAssetRegistry.h"
#endif

class FAutomationFilter;
class FAutomationGroupFilter;
struct FAutomatedTestFilter;
class SAutomationTestTagFilter;
class FAutomationTestPresetManager;
class FUICommandList;
class SAutomationWindowCommandBar;
class SCheckBox;
class SEditableTextBox;
struct FAutomationTestPreset;
template< typename ItemType > class TTextFilter;

/** Columns for the test tree view */
namespace AutomationTestWindowConstants
{
	const FName Checked( TEXT("Checked") );
	const FName Skipped( TEXT("Skipped") );
	const FName SkippedOptions( TEXT("SkippedOptions") );
	const FName Tags(TEXT("Tags"));
	const FName Title( TEXT("Name") );
	const FName SmokeTest( TEXT("SmokeTest") );
	const FName RequiredDeviceCount( TEXT("RequiredDeviceCount") );
	const FName Status( TEXT("Status") );
	const FName Timing( TEXT("Timing") );
}


/** The type of background style to use for the test list widget */
namespace EAutomationTestBackgroundStyle
{
	enum Type
	{
		Unknown,
		Editor,
		Game,
	};
}


/**
 * Implements the main UI Window for hosting all automation tests.
 */
class SAutomationWindow : public SCompoundWidget
{
	/**
	 * The automation text filter - used for updating the automation report list
	 */
	typedef TTextFilter< const TSharedPtr< class IAutomationReport >& > AutomationReportTextFilter;


	/** Single line in the automation output */
	struct FAutomationOutputMessage
	{
		/** Holds the message style. */
		FName Style;

		/** Holds the message text. */
		FString Text;

		/** Creates and initializes a new instance. */
		FAutomationOutputMessage (const FString& InText, const FName& InStyle)
			: Style(InStyle)
			, Text(InText)
		{ }
	};

public:

	SLATE_BEGIN_ARGS( SAutomationWindow ) { }
	SLATE_END_ARGS()

public:

	/** Default constructor. */
	SAutomationWindow();

	/** Destructor. */
	virtual ~SAutomationWindow();

public:

	/**
	 * Constructs the widget.
	 */
	void Construct(const FArguments& InArgs, const IAutomationControllerManagerRef& AutomationController, const TSharedRef<ISessionManager>& InSessionManager);

	/**
	* Check tests aren't running.
	*
	* @return true if the tests aren't running.
	*/
	bool IsAutomationControllerIdle() const;

	// Save the currently selected tests and expanded items so they can be reapplied later
	static TArray<FString> SaveExpandedTestNames(TSet<TSharedPtr<IAutomationReport>> ExpandedItems);

	// Expanded the given item if its name is in the array of strings given.
	static void ExpandItemsInList(TSharedPtr<STreeView<TSharedPtr<IAutomationReport>>> InTestTable, TSharedPtr<IAutomationReport> InReport, TArray<FString> ItemsToExpand);

	/** Override base class Tick */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:

	/**
	* Checks the list of selected rows to see if multiple rows are selected.
	*
	* @return true if multiple rows are selected.
	*/
	bool AreMultipleRowsSelected() {return TestTable->GetSelectedItems().Num()>1;}

	/**
	* Change the selection to a given row
	*
	* @param ThisRow a Report in the list of all reports
	*/
	void ChangeTheSelectionToThisRow(TSharedPtr< IAutomationReport > ThisRow);

	/**
	* Tests if the given row is in the list of selected rows.
	*
	* @param ThisRow a Report in the list of all report.
	* @return true if the report passed in is in the list of selected rows.
	*/
	bool IsRowSelected(TSharedPtr< IAutomationReport >  ThisRow);

	/**
	* Sets the enabled value of the selected rows to given value.
	*
	* @param InChecked Set all the selected rows to the enabled state to this value.
	*/
	void SetAllSelectedTestsChecked( bool InChecked );

	/**
	* Checks the list of selected rows to see if any are enabled.
	*
	* @return true if any of the selected rows are enabled.
	*/
	bool IsAnySelectedRowEnabled();

	/** Overridden from SWidget: Called when a key is pressed down - capturing copy. */
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

	/** Overridden from SWidget: Called after a key is released. */
	virtual FReply OnKeyUp( const FGeometry& InGeometry, const FKeyEvent& InKeyEvent ) override;

private:

	/**
	* Handle for a test item's check box being clicked.
	*
	* @param TestStatus The test row details.
	*/
	void HandleItemCheckBoxCheckedStateChanged( TSharedPtr< IAutomationReport > TestStatus );

	/** Callback for getting the enabled state of a test item. */
	bool HandleItemCheckBoxIsEnabled( ) const;

	/** Callback for getting the enabled state of the main content overlay. */
	bool HandleMainContentIsEnabled() const;

	/** Create the UI commands for the toolbar */
	void CreateCommands();

	/**
	 * Static: Creates a toolbar widget for the main automation window.
	 *
	 * @return The new widget.
	 */
	static TSharedRef< SWidget > MakeAutomationWindowToolBar( const TSharedRef<FUICommandList>& InCommandList, TSharedPtr<class SAutomationWindow> InLevelEditor );
	TSharedRef< SWidget > MakeAutomationWindowToolBar( const TSharedRef<FUICommandList>& InCommandList );

	/**
	 * Static: Creates a filter toolbar widget for the automation window.
	 *
	 * @return The new widget.
	 */
	static TSharedRef< SWidget > MakeAutomationFilterToolBar(const TSharedRef<FUICommandList>& InCommandList, TSharedPtr<class SAutomationWindow> InLevelEditor);
	TSharedRef< SWidget > MakeAutomationFilterToolBar(const TSharedRef<FUICommandList>& InCommandList);

	/**
	 * Static: Creates the test options menu widget.
	 *
	 * @return	The new widget.
	 */
	static TSharedRef< SWidget >GenerateTestsOptionsMenuContent( TWeakPtr<class SAutomationWindow> InAutomationWindow );
	TSharedRef< SWidget > GenerateTestsOptionsMenuContent( );

	/**
	 * Static: Creates the group flag options menu widget.
	 *
	 * @return The new widget.
	 */
	static TSharedRef< SWidget >GenerateGroupOptionsMenuContent( TWeakPtr<class SAutomationWindow> InAutomationWindow );
	TSharedRef< SWidget > GenerateGroupOptionsMenuContent( );

	/**
	 * Static: Creates the Presets menu widget.
	 *
	 * @return The new widget.
	 */
	static TSharedRef< SWidget >GeneratePresetsMenuContent( TWeakPtr<class SAutomationWindow> InAutomationWindow );
	TSharedRef< SWidget > GeneratePresetsMenuContent();

	/**
	* Static: Creates the test history options menu widget.
	*
	* @return The new widget.
	*/
	static TSharedRef< SWidget > GenerateTestHistoryMenuContent(TWeakPtr<class SAutomationWindow> InAutomationWindow);
	TSharedRef< SWidget > GenerateTestHistoryMenuContent();

	/**
	 * Creates a combo item for the requested filter.
	 *
	 * @return New combo item widget.
	 */
	TSharedRef<SWidget> GenerateRequestedFilterComboItem(TSharedPtr<FString> InItem);

	/**
	 * Creates a combo item for a test group.
	 *
	 * @return New combo item widget.
	 */
	TSharedRef<SWidget> GenerateGroupComboItem(TSharedPtr<FString> InItem);
		
	/**
	 * Populates OutSearchStrings with the strings that should be used in searching.
	 *
	 * @param Report The automation report to get a text description from.
	 * @param OutSearchStrings An array of stings to use.
	 */
	void PopulateReportSearchStrings( const TSharedPtr< IAutomationReport >& Report, OUT TArray< FString >& OutSearchStrings ) const;

	/** Callback for a test expansion changing */
	void OnExpansionChanged(TSharedPtr<IAutomationReport> InItem, bool bExpanded);

	/** Recursively expands subgroups as a reaction on expansion change (expands single-item subgroups without leafs recursively) */
	void ExpandSingleItemSubgroups(TSharedPtr<IAutomationReport> InItem, bool bExpanded);

	/** Gets children tests for a node in the hierarchy */
	void OnGetChildren(TSharedPtr<IAutomationReport> InItem, TArray<TSharedPtr<IAutomationReport> >& OutItems);

	/** Callback for a test expansion changing recursively */
	void OnTestExpansionRecursive(TSharedPtr<IAutomationReport> InTreeNode, bool bInIsItemExpanded);
	
	/** Callback for a new test being selected */
	void OnTestSelectionChanged(TSharedPtr<IAutomationReport> Selection, ESelectInfo::Type SelectInfo);

	/** Update the test log. */
	void UpdateTestLog(TSharedPtr<IAutomationReport> Selection);
	
	/** Called when the header checkbox's state is changed. Will go through all the Unit Tests available and change the state of their checkbox. */
	void HeaderCheckboxStateChange(ECheckBoxState InCheckboxState);	/** Creates table row for each visible automation test */
	
	/** Rebuilds the platform icon header */
	void RebuildPlatformIcons();
	
	/** Generate the device tooltip */
	FText CreateDeviceTooltip(int32 ClusterIndex);
	
	/** Clear the UI and icon header */
	void ClearAutomationUI ();

	/** Generates a row widget for a automation test */
	TSharedRef<ITableRow> OnGenerateWidgetForTest(TSharedPtr<IAutomationReport> InItem, const TSharedRef<STableViewBase>& OwnerTable );
	
	/** Generates a row widget for the log list view. */
	TSharedRef<ITableRow> OnGenerateWidgetForLog(TSharedPtr<FAutomationOutputMessage> Message, const TSharedRef<STableViewBase>& OwnerTable);

	/** Returns number of enabled tests (regardless of visibility) */
	FText OnGetNumEnabledTestsString() const;
	
	/** Returns number of workers in a device cluster */
	FText OnGetNumDevicesInClusterString(const int32 ClusterIndex) const;

	/** Callback when the list has been refreshed by the Automation Controller */
	void OnRefreshTestCallback();

	/** Finds available workers */
	void FindWorkers();

	//TODO AUTOMATION - remove if this step is automatic
	/** Updates list of all the tests */
	void ListTests();
	
	/** Goes through all selected tests and runs them. */
	FReply RunTests();

	/** Filter text has been updated */
	void OnFilterTextChanged( const FText& InFilterText );
	
	/** Returns if we're considering tests on content within the developer folders */
	bool IsDeveloperDirectoryIncluded() const;

	/** Toggles the consideration of tests within developer folders */
	void OnToggleDeveloperDirectoryIncluded();

	/** Returns if we're considering tests from exclude list only */
	bool IsExcludedTestsFilterOn() const;

	/** Toggles the consideration of tests from exclude list only */
	void OnToggleExcludedTestsFilter();

	/** Returns if we're filtering based on if the test is a "smoke" test */
	bool IsSmokeTestFilterOn() const;
	
	/** Toggles filtering of tests based on smoke test status */
	void OnToggleSmokeTestFilter();
	
	/** Returns if we're filtering based on if the test returned any warnings */
	bool IsWarningFilterOn() const;
	
	/** Toggles filtering of tests based on warning condition */
	void OnToggleWarningFilter();
	
	/** Returns if we're filtering based on if the test returned any errors */
	bool IsErrorFilterOn() const;
	
	/** Toggles filtering of tests based on error condition */
	void OnToggleErrorFilter();
	
	/** Returns if analytics should be sent to the back end */
	ECheckBoxState IsSendAnalyticsCheckBoxChecked() const;

	/** Toggles if we are sending analytics results from the tests */
	void HandleSendAnalyticsBoxCheckStateChanged(ECheckBoxState CheckBoxState);

	/** Returns if PIE should be kept open when test pass end */
	ECheckBoxState KeepPIEOpenCheckBoxChecked() const;

	/** Toggles if PIE should be kept open when test pass end */
	void HandleKeepPIEOpenBoxCheckStateChanged(ECheckBoxState CheckBoxState);

	/** Returns if we should automatically expand single-item test subgroups */
	ECheckBoxState AutoExpandSingleItemSubgroupsCheckBoxChecked() const;

	/** Toggles if automatic expansion of single-item subgroups is enabled */
	void HandleAutoExpandSingleItemSubgroupsCheckStateChanged(ECheckBoxState CheckBoxState);

	/** Returns if a device group is enabled */
	ECheckBoxState IsDeviceGroupCheckBoxIsChecked(const int32 DeviceGroupFlag) const;
	
	/** Toggles a device group flag */
	void HandleDeviceGroupCheckStateChanged(ECheckBoxState CheckBoxState, const int32 DeviceGroupFlag);

	/** Sitches presets in the Presets Menu of Preset ComboButton */
	void HandlePresetCheckStateChanged(ECheckBoxState CheckBoxState, const int32 EntryIndex, TSharedPtr<TArray<TSharedPtr<SCheckBox>>> CheckBoxes);

	/** Sets the number of times to repeat the tests */
	void OnChangeRepeatCount(int32 InNewValue);
	
	/** Returns the number of times to repeat the tests */
	int32 GetRepeatCount() const;

	/** Update the test list background style (Editor vs Game) */
	void UpdateTestListBackgroundStyle();

	/**
	* Gets the extention for the small brush, if enabled.
	*
	* @return The string to append to the brush path.
	*/
	FString GetSmallIconExtension() const;

	/**
	* Gets whether we should show large tool bar button details.
	*
	* @return The visibility setting for large tool bar details.
	*/
	EVisibility GetLargeToolBarVisibility() const;

	/**
	* Gets a brush for the automation start / stop state.
	*
	* @return - A start or stop icon - depending on state.
	*/
	const FSlateBrush* GetRunAutomationIcon() const;

	/**
	* Gets a label for the automation start / stop state.
	*
	* @return A start or stop tests label Depending on state.
	*/
	FText GetRunAutomationLabel() const;

	/**
	* Gets the brush to use for the test list background.
	*
	* @return The brush to use depending on the value of TestBackgroundType.
	*/
	const FSlateBrush* GetTestBackgroundBorderImage() const;

	/**
	* Recursively expand the tree nodes.
	*
	* @param InReport The current report.
	* @param ShouldExpand Should we expand the item.
	*/
	void ExpandTreeView( TSharedPtr< IAutomationReport > InReport, const bool ShouldExpand );

	/** Update the highlight string in the automation reports. */
	FText HandleAutomationHighlightText( ) const;

	/** Callback for determining the visibility of the 'Select a session' overlay. */
	EVisibility HandleSelectSessionOverlayVisibility( ) const;

	/** Callback for determining whether a session can be selected in the session manager. */
	void HandleSessionManagerCanSelectSession( const TSharedPtr<ISessionInfo>& Session, bool& CanSelect );

	/**
	 * Session selection has changed in the session manager
	 *
	 * @param SelectedSession The session that was selected.
	 */
	void HandleSessionManagerSelectionChanged( const TSharedPtr<ISessionInfo>& SelectedSession );

	/** Called when the session manager updates an instances. */
	void HandleSessionManagerInstanceChanged();

	/**
	 * Should the automation run button be enabled.
	 *
	 * @return true if it should be enabled.
	 */
	bool IsAutomationRunButtonEnabled() const;

	/**
	 * Set whether tests are available to run.
	 *
	 * @param The Automation controller state.
	 */
	void OnTestAvailableCallback( EAutomationControllerModuleState::Type InAutomationControllerState );

	/**
	 * Called when tests complete.
	 */
	void OnTestsCompleteCallback();

	/** Copies the selected log messages to the clipboard. */
	void CopyLog();

#if WITH_EDITOR
	/**
	 * Handle the context menu opening for automation reports.
	 *
	 * @return the context window.
	 */
	TSharedPtr<SWidget> HandleAutomationListContextMenuOpening( );

	// Added button for running the currently open level test.
	void RunSelectedTests();

	void FindTestReportsForCurrentEditorLevel(TArray<TSharedPtr<IAutomationReport>>& OutLevelReports);

	bool CanExecuteRunLevelTest();

	void OnRunLevelTest();

	void ScrollToTest(TSharedPtr<IAutomationReport> InReport);

	bool ExpandToTest(TSharedPtr<IAutomationReport> InRoot, TSharedPtr<IAutomationReport> InReport);
#endif

	/** Handles the save preset button being clicked. */
	FReply HandleSavePresetClicked();

	/** Handles the create new preset menu item being clicked. */
	void OnNewPresetClicked();

	/** Handles the remove preset menu item being clicked. */
	void OnRemovePresetClicked();

	/** Handles the rename preset menu item being clicked. */
	void OnRenamePresetClicked();

	/** Should the save preset button be enabled. */
	bool IsSaveButtonEnabled() const;

	/** Handles if the preset combo box should be visible. */
	EVisibility HandlePresetComboVisibility( ) const;

	/** Handles if the add preset text box should be visible. */
	EVisibility HandlePresetTextVisibility( ) const;

	/** Handles if the groups combo box should be visible. */
	EVisibility HandleGroupsVisibility() const;

	/** Called when the user commits the text in the add preset text box. */
	void HandlePresetTextCommited( const FText& CommittedText, ETextCommit::Type CommitType );

	/** Called when the user changes the requested test filter */
	void HandleRequesteFilterChanged(TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo);
	/** Called when the user changes the test group */
	void HandleGroupChanged(TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo);

	/** Expands the test tree to show all enabled tests. */
	void ExpandEnabledTests( TSharedPtr< IAutomationReport > InReport );

	/** Gets the text to display for the preset combo box. */
	FText GetPresetComboText() const;
	/** Gets the text to display for the requested filter combo box. */
	FText GetRequestedFilterComboText() const;
	/** Gets the combo box text to display for the selected group. */
	FText GetGroupComboText() const;

	/** Handle selected tests tag editing. */
	void HandleEditTagsButtonClicked() const;
	/** Whether the edit tags button is enabled. */
	bool IsEditTagsButtonEnabled() const;

	/**
	 * Handle the copy button clicked in the command bar.
	 *
	 * @return If the command was handled or not.
	 */
	FReply HandleCommandBarCopyLogClicked();

	/**
	 * Handle the log selection changed.
	 *
	 * @param The selected item.
	 * @param SelectInfo Provides context on how the selection changed.
	 */
	void HandleLogListSelectionChanged( TSharedPtr<FAutomationOutputMessage> InItem, ESelectInfo::Type SelectInfo );

	/**
	 * Gets the visibility of the test log window.
	 *
	 * @return The window's visibility.
	 * @see GetTestGraphVisibility
	 */
	EVisibility GetTestLogVisibility( ) const;

	/**
	 * Gets the visibility of the graphical test result window.
	 *
	 * @return The window's visibility.
	 * @see GetTestLogVisibility
	 */
	EVisibility GetTestGraphVisibility( ) const;

	/** Handles changing the display type for the graphical results window. */
	void HandleResultDisplayTypeStateChanged( ECheckBoxState NewRadioState, EAutomationGrapicalDisplayType::Type NewDisplayType );

	/** Handles checking if a display type is active for the graphical results window. */
	ECheckBoxState HandleResultDisplayTypeIsChecked( EAutomationGrapicalDisplayType::Type InDisplayType ) const;

	/** 
	 * Gets the visibility for the throbber.
	 *
	 * @return Whether we should show the throbber or not.
	 */
	EVisibility GetTestsUpdatingThrobberVisibility() const
	{
		return bIsRequestingTests ? EVisibility::Visible : EVisibility::Hidden;
	}

#if WITH_EDITOR
	void OnAssetRegistryFilesLoaded();
#endif

private:

	/** The automation window actions list. */
	TSharedPtr<FUICommandList> AutomationWindowActions;

	/** Holds a pointer to the active session. */
	TSharedPtr<ISessionInfo> ActiveSession;

	/** Holds the AutomationController. */
	IAutomationControllerManagerPtr AutomationController;

	/** Holds the search box widget. */
	TSharedPtr< SSearchBox > AutomationSearchBox;

	/** Must maintain a widget size so the header and row icons can line up. */
	const float ColumnWidth;

	/** The list of all valid tests. */
	TSharedPtr< STreeView< TSharedPtr< IAutomationReport > > > TestTable;
	
	/** Widget for header platform icons. */
	TSharedPtr< SHorizontalBox > PlatformsHBox;

	/** Widget for the command bar. */
	TSharedPtr< SAutomationWindowCommandBar > CommandBar;

	/** Widget for the menu bar - run automation etc. */
	TSharedPtr< SVerticalBox > MenuBar;

	/** Holds the widget to display log messages. */
	TSharedPtr<SListView<TSharedPtr<FAutomationOutputMessage> > > LogListView;

	/** Holds the widget to display a graph of the results. */
	TSharedPtr< SAutomationGraphicalResultBox > GraphicalResultBox;

	/** Holds the collection of log messages. */
	TArray<TSharedPtr<FAutomationOutputMessage> > LogMessages;

	/** The automation report text filter. */
	TSharedPtr< AutomationReportTextFilter > AutomationTextFilter;

	/** The automation general filter - for smoke tests / warnings and Errors. */
	TSharedPtr< FAutomationFilter > AutomationGeneralFilter;

	/** The automation group filter - filters tests based on selected group filter. */
	TSharedPtr< FAutomationGroupFilter > AutomationGroupFilter;

	/** The automation tag filter - filters tests based on tag expression. */
	TSharedPtr< FAutomationGroupFilter > AutomationTagFilter;

	/** The automation filter collection - contains the automation filters. */
	TSharedPtr< AutomationFilterCollection > AutomationFilters;

	/** Holds the session manager. */
	TSharedPtr<ISessionManager> SessionManager;

	/**
	 * Holds the automation controller module state.
	 * This is set by the automation controller callback. We may go back to querying the module directly.
	 */
	EAutomationControllerModuleState::Type AutomationControllerState;

	/** Flag to acknowledge if the window is awaiting tests to display. */
	bool bIsRequestingTests;

	/** Flag to tell if we have a child test selected in the test tree. */
	bool bHasChildTestSelected;

	/** Which type of window style to use for the test background. */
	EAutomationTestBackgroundStyle::Type TestBackgroundType;

	/** True if we are creating a new preset (The add/rename preset text box is visible). */
	bool bAddingTestPreset;

	/** True if we are renaming a new preset (The add/rename preset text box is visible). */
	bool bRenamingTestPreset;

	/** Flag to enable display of the text labels in the toolbar. */
	bool bIsLabelVisibilityEnabled;

	/** Flag to enable auto expand of subgroups that have single-item groups without children. */
	bool bAutoExpandSingleItemSubgroups;

	/** Holds a pointer to the preset manager. */
	TSharedPtr<FAutomationTestPresetManager> TestPresetManager;

	/** Holds the currently selected preset. */
	TSharedPtr<FAutomationTestPreset> SelectedPreset;

	/** Holds a pointer to the preset combo button widget. */
	TSharedPtr< STextBlock > PresetComboButtonText;

	/** Holds a pointer to the preset combo box widget. */
	TSharedPtr< SComboBox< TSharedPtr<FString> > >	RequestedFilterComboBox;
	TArray< TSharedPtr< FString > >					RequestedFilterComboList;

	/** Holds a pointer to the groups combo box widget. */
	TSharedPtr< SComboBox< TSharedPtr<FString> > >	GroupComboBox;
	TArray< TSharedPtr< FString > >					GroupComboList;

	/** Map for fast access of test group filters by name. */
	TMap< FString, TArray<FAutomatedTestFilter> > GroupFiltersMap;

	/** Holds a pointer to the preset text box. */
	TSharedPtr<SEditableTextBox> PresetTextBox;

	/** List of expanded test items to preserve during a refresh */
	TArray<FString> SavedExpandedItems;

	// Saves the last selected test so that it can be unbinded from refreshing the log
	TWeakPtr<IAutomationReport> PreviousSelection;

	/** Holds a pointer to the tag filter widget. */
	TSharedPtr<SAutomationTestTagFilter> TagFilter;
};
