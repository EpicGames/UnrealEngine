// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAutomationWindowCommandBar.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "SAutomationExportMenu.h"

#define LOCTEXT_NAMESPACE "SAutomationWindowCommandBar"


/* SAutomationWindowCommandBar interface
 *****************************************************************************/

void SAutomationWindowCommandBar::Construct( const FArguments& InArgs, const TSharedRef< SNotificationList >& InNotificationList )
{
 	OnCopyLogClicked = InArgs._OnCopyLogClicked;

	ChildSlot
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
			.FillWidth(1.0f)
		
		+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				SNew(SAutomationExportMenu, InNotificationList)
				.Visibility(this, &SAutomationWindowCommandBar::GetExportMenuVisibility)
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				// copy button
				SAssignNew(CopyButton, SButton)
					.Visibility(this, &SAutomationWindowCommandBar::GetCopyButtonVisibility)
					.ContentPadding(FMargin(6.0, 2.0))
					.IsEnabled(false)
					.Text(LOCTEXT("AutomationCopyButtonText", "Copy"))
					.ToolTipText(LOCTEXT("AutomationCopyButtonTooltip", "Copy the selected log messages to the clipboard"))
					.OnClicked(this, &SAutomationWindowCommandBar::HandleCopyButtonClicked)
			]
	];
}


void SAutomationWindowCommandBar::SetCopyButtonVisibility(EVisibility InVisibility)
{
	CopyButtonVisibility = (InVisibility == EVisibility::Visible) ? EVisibility::Visible : EVisibility::Collapsed;
}


EVisibility SAutomationWindowCommandBar::GetCopyButtonVisibility() const
{
	return CopyButtonVisibility;
}


EVisibility SAutomationWindowCommandBar::GetExportMenuVisibility() const
{
	return (CopyButtonVisibility == EVisibility::Visible) ? EVisibility::Hidden : EVisibility::Visible;
}

FReply SAutomationWindowCommandBar::HandleCopyButtonClicked( )
{
	if (OnCopyLogClicked.IsBound())
	{
		OnCopyLogClicked.Execute();
	}

	return FReply::Handled();
}


void SAutomationWindowCommandBar::SetNumLogMessages( int32 Count )
{
	CopyButton->SetEnabled(Count > 0);
}


#undef LOCTEXT_NAMESPACE
