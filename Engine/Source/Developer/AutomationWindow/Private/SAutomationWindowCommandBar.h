// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"

class SButton;

/**
 * Implements the automation console command bar widget.
 */
class SAutomationWindowCommandBar
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SAutomationWindowCommandBar) { }

 		/** Called when the copy log button is clicked. */
 		SLATE_EVENT(FOnClicked, OnCopyLogClicked)

	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef< SNotificationList >& InNotificationList );

	/**
	 * Sets the number of messages selected in the log window.
	 *
	 * @param Count Number of selected messages.
	 */
	void SetNumLogMessages( int Count );

	/** Sets visibility status for Copy button */
	void SetCopyButtonVisibility(EVisibility InVisibility);

private:

	/** Handles clicking the copy log button. */
	FReply HandleCopyButtonClicked( );

	EVisibility GetCopyButtonVisibility() const;
	EVisibility GetExportMenuVisibility() const;

private:

 	/** Holds the copy log button. */
 	TSharedPtr<SButton> CopyButton;

 	/** Holds a delegate that is executed when the copy log button is clicked. */
 	FOnClicked OnCopyLogClicked;

	/** Holds visibility status for Copy button */
	EVisibility CopyButtonVisibility = EVisibility::Collapsed;
};
