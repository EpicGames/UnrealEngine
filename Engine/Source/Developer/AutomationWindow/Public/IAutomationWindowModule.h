// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Modules/ModuleInterface.h"
#include "IAutomationControllerManager.h"

class ISessionManager;

/** Delegate to call when the automation window module is shutdown. */
DECLARE_DELEGATE(FOnAutomationWindowModuleShutdown);


/**
 * Automation Window module
 */
class IAutomationWindowModule
	: public IModuleInterface
{
public:

	/**
	 * Creates a Automation Window widget.
	 *
	 * @param AutomationController The automation controller to use.
	 * @param SessionManager The session manager to use.
	 * @return	New Automation Test window widget
	 */
	virtual TSharedRef<class SWidget> CreateAutomationWindow( const IAutomationControllerManagerRef& AutomationController, const TSharedRef<ISessionManager>& SessionManager ) = 0;

	/**
	 * Store a pointer to the AutomationWindowTab.
	 *
	 * @return a weak pointer to the automation tab.
	 */
	virtual TWeakPtr<class SDockTab> GetAutomationWindowTab()  = 0;

	/**
	 * Store a pointer to the AutomationWindowTab.
	 *
	 * @param AutomationWindowTab a weak pointer to the automation tab.
	 */
	virtual void SetAutomationWindowTab(TWeakPtr<class SDockTab> AutomationWindowTab) = 0;

public:

	/**
	 * Gets a delegate that is invoked when the automation window module shuts down.
	 *
	 * @return The delegate.
	 */
	virtual FOnAutomationWindowModuleShutdown& OnShutdown( ) = 0;

protected:

	/** Virtual destructor. */
	~IAutomationWindowModule( ) = default;
};
